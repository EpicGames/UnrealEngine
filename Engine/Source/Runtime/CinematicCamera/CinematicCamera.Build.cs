// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CinematicCamera : ModuleRules
{
	public CinematicCamera(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"DeveloperSettings",
                "Engine",
                "MovieScene",
                "MovieSceneTracks",
                "Slate",
                "SlateCore"
			}
		);
	}
}
