// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraRig_Crane.h"
#include "UObject/ConstructorHelpers.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/CollisionProfile.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraRig_Crane)

#define LOCTEXT_NAMESPACE "CameraRig_Crane"

ACameraRig_Crane::ACameraRig_Crane(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryActorTick.bCanEverTick = true;

	// default control values
	CraneYaw = 0.f;
	CranePitch = 0.f;
	CraneArmLength = 500.f;
	bLockMountPitch = false;
	bLockMountYaw = false;

	// create the root component
	TransformComponent = CreateDefaultSubobject<USceneComponent>(TEXT("TransformComponent"));
	RootComponent = TransformComponent;
	
	// create the functional component hierarchy
	CraneYawControl = CreateDefaultSubobject<USceneComponent>(TEXT("CraneYawControl"));
	CraneYawControl->SetupAttachment(TransformComponent);
	CraneYawControl->SetRelativeLocation(FVector(0.f, 0.f, 70.f));			// pivot height off the ground
	CraneYawControl->SetRelativeRotation(FRotator(0.f, CraneYaw, 0.f));

	CranePitchControl = CreateDefaultSubobject<USceneComponent>(TEXT("CranePitchControl"));
	CranePitchControl->SetupAttachment(CraneYawControl);
	CranePitchControl->SetRelativeRotation(FRotator(CranePitch, 0.f, 0.f));

	CraneCameraMount = CreateDefaultSubobject<USceneComponent>(TEXT("CraneCameraMount"));
	CraneCameraMount->SetupAttachment(CranePitchControl);
	CraneCameraMount->SetRelativeLocation(FVector(CraneArmLength, 0.f, -15.f));			// negative z == underslung mount

#if WITH_EDITORONLY_DATA
	// create preview meshes
	if (!IsRunningDedicatedServer())
	{
		static ConstructorHelpers::FObjectFinder<UStaticMesh> CraneBaseMesh(TEXT("/Engine/EditorMeshes/Camera/SM_CraneRig_Base.SM_CraneRig_Base"));
		PreviewMesh_CraneBase = CreateOptionalDefaultSubobject<UStaticMeshComponent>(TEXT("PreviewMesh_CraneBase"));
		if (PreviewMesh_CraneBase)
		{
			PreviewMesh_CraneBase->SetCanEverAffectNavigation(false);
			PreviewMesh_CraneBase->SetStaticMesh(CraneBaseMesh.Object);
			PreviewMesh_CraneBase->SetIsVisualizationComponent(true);
			PreviewMesh_CraneBase->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
			PreviewMesh_CraneBase->bHiddenInGame = true;
			PreviewMesh_CraneBase->CastShadow = false;

			PreviewMesh_CraneBase->SetupAttachment(TransformComponent);		// sibling of yawcontrol
		}

		static ConstructorHelpers::FObjectFinder<UStaticMesh> CraneArmMesh(TEXT("/Engine/EditorMeshes/Camera/SM_CraneRig_Arm.SM_CraneRig_Arm"));
		PreviewMesh_CraneArm = CreateOptionalDefaultSubobject<UStaticMeshComponent>(TEXT("PreviewMesh_CraneArm"));
		if (PreviewMesh_CraneArm)
		{
			PreviewMesh_CraneArm->SetCanEverAffectNavigation(false);
			PreviewMesh_CraneArm->SetStaticMesh(CraneArmMesh.Object);
			PreviewMesh_CraneArm->SetIsVisualizationComponent(true);
			PreviewMesh_CraneArm->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
			PreviewMesh_CraneArm->bHiddenInGame = true;
			PreviewMesh_CraneArm->CastShadow = false;
			
			PreviewMesh_CraneArm->SetupAttachment(CranePitchControl);		// sibling of the mount
			PreviewMesh_CraneArm->SetRelativeRotation(FRotator(0.f, 90.f, 0.f));
			PreviewMesh_CraneArm->SetRelativeLocation(FVector(0.f, 0.f, 52.f));
			PreviewMesh_CraneArm->SetRelativeScale3D(FVector(0.7f, 0.7f, 0.7f));
		}
		
		static ConstructorHelpers::FObjectFinder<UStaticMesh> CraneArmMount(TEXT("/Engine/EditorMeshes/Camera/SM_CraneRig_Mount.SM_CraneRig_Mount"));
		PreviewMesh_CraneMount = CreateOptionalDefaultSubobject<UStaticMeshComponent>(TEXT("PreviewMesh_CraneMount"));
		if (PreviewMesh_CraneMount)
		{
			PreviewMesh_CraneMount->SetCanEverAffectNavigation(false);
			PreviewMesh_CraneMount->SetStaticMesh(CraneArmMount.Object);
			PreviewMesh_CraneMount->SetIsVisualizationComponent(true);
			PreviewMesh_CraneMount->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
			PreviewMesh_CraneMount->bHiddenInGame = true;
			PreviewMesh_CraneMount->CastShadow = false;

			PreviewMesh_CraneMount->SetupAttachment(CranePitchControl);
			PreviewMesh_CraneMount->SetRelativeRotation(FRotator(0.f, 90.f, 0.f));
			PreviewMesh_CraneMount->SetRelativeLocation(FVector(CraneArmLength, 0.f, 0.f));
			PreviewMesh_CraneMount->SetRelativeScale3D(FVector(0.7f, 0.7f, 0.7f));
		}
		
		static ConstructorHelpers::FObjectFinder<UStaticMesh> CraneArmCW(TEXT("/Engine/EditorMeshes/Camera/SM_CraneRig_Body.SM_CraneRig_Body"));
		PreviewMesh_CraneCounterWeight = CreateOptionalDefaultSubobject<UStaticMeshComponent>(TEXT("PreviewMesh_CraneCounterWeight"));
		if (PreviewMesh_CraneCounterWeight)
		{
			PreviewMesh_CraneCounterWeight->SetCanEverAffectNavigation(false);
			PreviewMesh_CraneCounterWeight->SetStaticMesh(CraneArmCW.Object);
			PreviewMesh_CraneCounterWeight->SetIsVisualizationComponent(true);
			PreviewMesh_CraneCounterWeight->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
			PreviewMesh_CraneCounterWeight->bHiddenInGame = true;
			PreviewMesh_CraneCounterWeight->CastShadow = false;

			PreviewMesh_CraneCounterWeight->SetupAttachment(CranePitchControl);
			PreviewMesh_CraneCounterWeight->SetRelativeRotation(FRotator(0.f, 90.f, 0.f));
			PreviewMesh_CraneCounterWeight->SetRelativeScale3D(FVector(0.7f, 0.7f, 0.7f));
		}

		UpdatePreviewMeshes();
	}
#endif
}

static const float CraneArmMesh_DefaultMeshSize = 29.f * 0.7f;		// size of the mesh in the dimension that will stretch (accounting for the 0.7 scale)

#if WITH_EDITORONLY_DATA
void ACameraRig_Crane::UpdatePreviewMeshes()
{
	if (PreviewMesh_CraneArm)
	{
		// note to explain the math here:
		// this tailored to the dimensions of the asset (29cm in scaling direction, pivot at the edge)
		float const CraneArmYScale = CraneArmLength / CraneArmMesh_DefaultMeshSize;

		FVector NewRelScale3D = PreviewMesh_CraneArm->GetRelativeScale3D();
		NewRelScale3D.Y = CraneArmYScale * 0.7f;

		PreviewMesh_CraneArm->SetRelativeScale3D(NewRelScale3D);
	}

	if (PreviewMesh_CraneMount)
	{
		const float ZOffset = 40.f;		// make it appear in the right place
		FVector const NewLoc(CraneArmLength, 0.f, ZOffset);
		PreviewMesh_CraneMount->SetRelativeLocation(NewLoc);
	}
}
#endif

void ACameraRig_Crane::UpdateCraneComponents()
{
	FRotator NewYawControlRot = CraneYawControl->GetRelativeRotation();
	NewYawControlRot.Yaw = CraneYaw;
	CraneYawControl->SetRelativeRotation(FRotator(0.f, CraneYaw, 0.f));

	FRotator NewPitchControlRot = CranePitchControl->GetRelativeRotation();
	NewPitchControlRot.Pitch = CranePitch;
	CranePitchControl->SetRelativeRotation(FRotator(CranePitch, 0.f, 0.f));

	FVector NewCameraMountLoc = CraneCameraMount->GetRelativeLocation();
	NewCameraMountLoc.X = CraneArmLength;

	// zero the pitch from the camera mount component
	// this effectively gives us bAbsoluteRotation for only pitch component of an attached camera
	FRotator NewCameraMountWorldRot = CraneCameraMount->GetAttachParent() ? CraneCameraMount->GetAttachParent()->GetComponentRotation() : FRotator(0.f, 0.f, 0.f);
	if (!bLockMountPitch)
	{
		NewCameraMountWorldRot.Pitch = 0.f;
	}
	if (!bLockMountYaw)
	{
		NewCameraMountWorldRot.Yaw = RootComponent->GetRelativeRotation().Yaw;
	}
	NewCameraMountWorldRot.Roll = 0.f;

	FQuat RelativeRot = CraneCameraMount->GetRelativeRotationFromWorld(FQuat(NewCameraMountWorldRot));
	CraneCameraMount->SetRelativeTransform(FTransform(RelativeRot, NewCameraMountLoc));

#if WITH_EDITORONLY_DATA
	UpdatePreviewMeshes();
#endif
}


void ACameraRig_Crane::Tick(float DeltaTime)
{
	Super::Tick(DeltaTime);

	// feed exposed API into underlying components
	UpdateCraneComponents();
}

#if WITH_EDITOR
void ACameraRig_Crane::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	UpdateCraneComponents();
}

void ACameraRig_Crane::PostEditUndo()
{
	Super::PostEditUndo();

	UpdateCraneComponents();
}
#endif // WITH_EDITOR

USceneComponent* ACameraRig_Crane::GetDefaultAttachComponent() const
{
	return CraneCameraMount;
}

bool ACameraRig_Crane::ShouldTickIfViewportsOnly() const
{
	return true;
}


#undef LOCTEXT_NAMESPACE

