// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraRig_Rail.h"
#include "UObject/ConstructorHelpers.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/CollisionProfile.h"
#include "Engine/World.h"
#include "Components/SplineComponent.h"
#include "Components/SplineMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraRig_Rail)

#define LOCTEXT_NAMESPACE "CameraRig_Rail"

ACameraRig_Rail::ACameraRig_Rail(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.bStartWithTickEnabled = true;

	CurrentPositionOnRail = 0.f;
	bLockOrientationToRail = false;

#if WITH_EDITORONLY_DATA
	bShowRailVisualization = true;
	PreviewMeshScale = 1.f;
#endif

	// Create components
	TransformComponent = CreateDefaultSubobject<USceneComponent>(TEXT("TransformComponent"));

	// Make the scene component the root component
	RootComponent = TransformComponent;

	RailSplineComponent = CreateDefaultSubobject<USplineComponent>(TEXT("RailSplineComponent"));
	RailSplineComponent->SetupAttachment(TransformComponent);

	RailCameraMount = CreateDefaultSubobject<USceneComponent>(TEXT("RailCameraMount"));
	RailCameraMount->SetupAttachment(RailSplineComponent);

#if WITH_EDITORONLY_DATA
	if (!IsRunningDedicatedServer())
	{
		static ConstructorHelpers::FObjectFinder<UStaticMesh> RailMesh(TEXT("/Engine/EditorMeshes/Camera/SM_RailRig_Track.SM_RailRig_Track"));
		PreviewRailStaticMesh = RailMesh.Object;

		static ConstructorHelpers::FObjectFinder<UStaticMesh> MountMesh(TEXT("/Engine/EditorMeshes/Camera/SM_RailRig_Mount.SM_RailRig_Mount"));
		PreviewMesh_Mount = CreateOptionalDefaultSubobject<UStaticMeshComponent>(TEXT("PreviewMesh_Mount"));
		if (PreviewMesh_Mount)
		{
			PreviewMesh_Mount->SetCanEverAffectNavigation(false);
			PreviewMesh_Mount->SetStaticMesh(MountMesh.Object);
			PreviewMesh_Mount->SetIsVisualizationComponent(true);
			PreviewMesh_Mount->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
			PreviewMesh_Mount->bHiddenInGame = true;
			PreviewMesh_Mount->CastShadow = false;

			PreviewMesh_Mount->SetupAttachment(RailCameraMount);
		}
	}
#endif
}

#if WITH_EDITORONLY_DATA
USplineMeshComponent* ACameraRig_Rail::CreateSplinePreviewSegment()
{
	USplineMeshComponent* const Segment = NewObject<USplineMeshComponent>(this);
	if (Segment)
	{
		Segment->SetStaticMesh(PreviewRailStaticMesh);
		Segment->SetMobility(EComponentMobility::Movable);
		Segment->SetIsVisualizationComponent(true);
		Segment->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
		Segment->bHiddenInGame = true;
		Segment->CastShadow = false;
	 
		Segment->SetupAttachment(TransformComponent);
		Segment->RegisterComponent();
	}

	return Segment;
}

void ACameraRig_Rail::UpdatePreviewMeshes()
{
	if (RailSplineComponent)
	{
		if (PreviewRailStaticMesh)
		{
			int32 const NumNeededPreviewMeshes = FMath::Max(0, RailSplineComponent->GetNumberOfSplineSegments());

			// make sure our preview mesh array is correctly sized and populated
			{
				PreviewRailMeshSegments.RemoveAll([](USplineMeshComponent* Segment) { return Segment == nullptr; });

				int32 const NumExistingPreviewMeshes = PreviewRailMeshSegments.Num();
				if (NumExistingPreviewMeshes > NumNeededPreviewMeshes)
				{
					// we have too many meshes, remove some
					int32 const NumToRemove = NumExistingPreviewMeshes - NumNeededPreviewMeshes;
					for (int Idx = 0; Idx < NumToRemove; ++Idx)
					{
						USplineMeshComponent* const ElementToRemove = PreviewRailMeshSegments.Pop();
						ElementToRemove->UnregisterComponent();
					}
				}
				else if (NumExistingPreviewMeshes < NumNeededPreviewMeshes)
				{
					int32 const NumToAdd = NumNeededPreviewMeshes - NumExistingPreviewMeshes;

					for (int32 Idx = 0; Idx < NumToAdd; ++Idx)
					{
						USplineMeshComponent* PreviewMesh = CreateSplinePreviewSegment();
						PreviewRailMeshSegments.Add(PreviewMesh);
					}
				}
				check(PreviewRailMeshSegments.Num() == NumNeededPreviewMeshes);
			}

			for (int PtIdx = 0; PtIdx < NumNeededPreviewMeshes; ++PtIdx)
			{
				FVector StartLoc = RailSplineComponent->GetLocationAtSplinePoint(PtIdx, ESplineCoordinateSpace::Local);
				FVector StartTangent = RailSplineComponent->GetLeaveTangentAtSplinePoint(PtIdx, ESplineCoordinateSpace::Local);
				FVector EndTangent = RailSplineComponent->GetArriveTangentAtSplinePoint(PtIdx + 1,ESplineCoordinateSpace::Local);
				FVector EndLoc = RailSplineComponent->GetLocationAtSplinePoint(PtIdx + 1, ESplineCoordinateSpace::Local);

				USplineMeshComponent* const SplineMeshComp = PreviewRailMeshSegments[PtIdx];
				if (SplineMeshComp)
				{
					SplineMeshComp->SetVisibility(bShowRailVisualization,false);
					SplineMeshComp->SetStartScale(FVector2D(PreviewMeshScale, PreviewMeshScale),false);
					SplineMeshComp->SetEndScale(FVector2D(PreviewMeshScale, PreviewMeshScale),false);
					SplineMeshComp->SetForwardAxis(ESplineMeshAxis::Z,false);
					SplineMeshComp->SetStartAndEnd(StartLoc, StartTangent, EndLoc, EndTangent, false);
					SplineMeshComp->UpdateMesh();
				}
			}

			// Unregister any owned components that aren't in PreviewRailMeshSegments
			TArray<USplineMeshComponent*> OwnedSplineMeshComponents;
			GetComponents(OwnedSplineMeshComponents);
			for (USplineMeshComponent* OwnedComponent : OwnedSplineMeshComponents)
			{
				if (!PreviewRailMeshSegments.Contains(OwnedComponent) && OwnedComponent->IsRegistered())
				{
					OwnedComponent->UnregisterComponent();
				}
			}
		}
		
		// make visualization of the mount follow the contour of the rail
		if (PreviewMesh_Mount)
		{
			PreviewMesh_Mount->SetVisibility(bShowRailVisualization);
			PreviewMesh_Mount->SetWorldScale3D(FVector(PreviewMeshScale, PreviewMeshScale, PreviewMeshScale));
		}
	}
}
#endif

void ACameraRig_Rail::UpdateRailComponents()
{
	if (!GetWorld())
	{
		return;
	}

	if (RailSplineComponent && RailCameraMount)
	{
		float const SplineLen = RailSplineComponent->GetSplineLength();
		FVector const MountPos = RailSplineComponent->GetLocationAtDistanceAlongSpline(CurrentPositionOnRail*SplineLen, ESplineCoordinateSpace::World);

		if (bLockOrientationToRail)
		{
			FQuat const RailRot = RailSplineComponent->GetQuaternionAtDistanceAlongSpline(CurrentPositionOnRail*SplineLen, ESplineCoordinateSpace::World);
			RailCameraMount->SetWorldTransform(FTransform(RailRot, MountPos));
		}
		else
		{
			RailCameraMount->SetWorldLocation(MountPos);
		}
	}

#if WITH_EDITOR
	if (GIsEditor)
	{
		const UWorld* const MyWorld = GetWorld();
		if (MyWorld && !MyWorld->IsGameWorld())
		{
			// set up preview mesh to match #todo
			UpdatePreviewMeshes();
		}
	}
#endif
}

USceneComponent* ACameraRig_Rail::GetDefaultAttachComponent() const
{
	return RailCameraMount;
}

#if WITH_EDITOR
void ACameraRig_Rail::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	UpdateRailComponents();
}

void ACameraRig_Rail::PostEditUndo()
{
	Super::PostEditUndo();
	UpdateRailComponents();
}

void ACameraRig_Rail::PostEditMove(bool bFinished)
{
	Super::PostEditMove(bFinished);
	UpdateRailComponents();
}
#endif

void ACameraRig_Rail::Tick(float DeltaTime)
{
	Super::Tick(DeltaTime);

	// feed exposed API into underlying components
	UpdateRailComponents();
}

bool ACameraRig_Rail::ShouldTickIfViewportsOnly() const
{
	return true;
}

#undef LOCTEXT_NAMESPACE

