// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineCameraActor.h"
#include "DrawDebugHelpers.h"
#include "CineCameraComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CineCameraActor)

#define LOCTEXT_NAMESPACE "CineCameraActor"

//////////////////////////////////////////////////////////////////////////
// ACameraActor

ACineCameraActor::ACineCameraActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer
			.SetDefaultSubobjectClass<UCineCameraComponent>(TEXT("CameraComponent"))
	)
{
	CineCameraComponent = Cast<UCineCameraComponent>(GetCameraComponent());

	PrimaryActorTick.bCanEverTick = true;
	SetActorTickEnabled(true);

#if WITH_EDITORONLY_DATA
	bIsSpatiallyLoaded = false;
#endif
}

void ACineCameraActor::PostInitializeComponents()
{
	Super::PostInitializeComponents();
	LookatTrackingSettings.LastLookatTrackingRotation = GetActorRotation();
}

bool ACineCameraActor::ShouldTickIfViewportsOnly() const
{
	return true;
}

FVector ACineCameraActor::GetLookatLocation() const
{
	FVector FinalLookat;
	if (AActor* ActorToTrack = LookatTrackingSettings.ActorToTrack.Get())
	{
		FTransform const BaseTransform = ActorToTrack->GetActorTransform();
		FinalLookat = BaseTransform.TransformPosition(LookatTrackingSettings.RelativeOffset);
	}
	else
	{
		FinalLookat = LookatTrackingSettings.RelativeOffset;
	}

	return FinalLookat;
}

static const FColor DebugLookatTrackingPointSolidColor(200, 200, 32, 128);		// yellow
static const FColor DebugLookatTrackingPointOutlineColor = FColor::Black;

void ACineCameraActor::Tick(float DeltaTime)
{
	Super::Tick(DeltaTime);

	if (GetCameraComponent() && ShouldTickForTracking())
	{
		if (LookatTrackingSettings.bEnableLookAtTracking)
		{
			// do the lookat tracking
			// #note this will turn the whole actor, which assumes the cameracomponent's transform is the same as the root component
			// more complex component hierarchies will require different handling here
			FVector const LookatLoc = GetLookatLocation();
			FVector const ToLookat = LookatLoc - GetActorLocation();
			FRotator FinalRot = 
				bResetInterplation
				? ToLookat.Rotation()
				: FMath::RInterpTo(LookatTrackingSettings.LastLookatTrackingRotation, ToLookat.Rotation(), DeltaTime, LookatTrackingSettings.LookAtTrackingInterpSpeed);

			if (LookatTrackingSettings.bAllowRoll)
			{
				FinalRot.Roll = GetActorRotation().Roll;
			}

			SetActorRotation(FinalRot);

			// we store this ourselves in case other systems try to change our rotation, and end fighting the interpolation
			LookatTrackingSettings.LastLookatTrackingRotation = FinalRot;
			// the frame as well, so the CameraComponent can fight those other systems.
			// This is a bug fix for when starting a new level sequence forces a re-flush of all timelines
			// including camera transforms.  That's probably the only other system referenced above.
			LookatTrackingSettings.LastLookatTrackingRotationFrame = GFrameNumber;
			
#if ENABLE_DRAW_DEBUG
			if (LookatTrackingSettings.bDrawDebugLookAtTrackingPosition)
			{
				::DrawDebugSolidBox(GetWorld(), LookatLoc, FVector(12.f), DebugLookatTrackingPointSolidColor);
				::DrawDebugBox(GetWorld(), LookatLoc, FVector(12.f), DebugLookatTrackingPointOutlineColor);
			}
#endif // ENABLE_DRAW_DEBUG
		}
	}

	bResetInterplation = false;
}


void ACineCameraActor::NotifyCameraCut()
{
	Super::NotifyCameraCut();
	
	bResetInterplation = true;
}

bool ACineCameraActor::ShouldTickForTracking() const
{
	bool bShouldTickForTracking = 
		LookatTrackingSettings.bEnableLookAtTracking || 
		CineCameraComponent->FocusSettings.TrackingFocusSettings.bDrawDebugTrackingFocusPoint;

#if WITH_EDITORONLY_DATA
	if (CineCameraComponent->FocusSettings.bDrawDebugFocusPlane)
	{
		bShouldTickForTracking = true;
	}
#endif // WITH_EDITORONLY_DATA

	return bShouldTickForTracking;
}


#undef LOCTEXT_NAMESPACE

