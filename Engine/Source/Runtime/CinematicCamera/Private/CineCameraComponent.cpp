// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineCameraComponent.h"
#include "UObject/CineCameraObjectVersion.h"
#include "UObject/ReleaseObjectVersion.h"
#include "UObject/ConstructorHelpers.h"
#include "GameFramework/Actor.h"
#include "Engine/World.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/CollisionProfile.h"
#include "Materials/Material.h"
#include "GameFramework/WorldSettings.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "DrawDebugHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CineCameraComponent)

#define LOCTEXT_NAMESPACE "CineCameraComponent"


//////////////////////////////////////////////////////////////////////////
// UCameraComponent

/// @cond DOXYGEN_WARNINGS

UCineCameraComponent::UCineCameraComponent()
{
	// Super 35mm 4 Perf
	// Default filmback and lens settings will be overridden if valid default presets are specified in ini
	

#if WITH_EDITORONLY_DATA
	bTickInEditor = true;
#endif
	
	// NOTE: in UE 5.3 and onward, classes deriving from this component must explicitly register for tick if
	// overriding the tick function, since this class only ticks productively with the following defines on
#if WITH_EDITORONLY_DATA || ENABLE_DRAW_DEBUG
	PrimaryComponentTick.bCanEverTick = true;
#endif

	bAutoActivate = true;

	bConstrainAspectRatio = true;

	
	const UCineCameraSettings* CineCameraSettings = GetDefault<UCineCameraSettings>();
	check(CineCameraSettings);
	// default filmback
	SetFilmbackPresetByNameInternal(CineCameraSettings->DefaultFilmbackPreset, Filmback);
	SetFilmbackPresetByNameInternal(CineCameraSettings->DefaultFilmbackPreset, FilmbackSettings_DEPRECATED);
	SetLensPresetByNameInternal(CineCameraSettings->DefaultLensPresetName);
	SetCropPresetByNameInternal(CineCameraSettings->DefaultCropPresetName);
	// other lens defaults
	CurrentAperture = CineCameraSettings->DefaultLensFStop;
	CurrentFocalLength = CineCameraSettings->DefaultLensFocalLength;

	ExposureMethod = ECameraExposureMethod::Enabled;

	RecalcDerivedData();

#if WITH_EDITORONLY_DATA
	if (!IsRunningCommandlet())
	{
		// overrides CameraComponent's camera mesh
		static ConstructorHelpers::FObjectFinder<UStaticMesh> EditorCameraMesh(TEXT("/Engine/EditorMeshes/Camera/SM_CineCam.SM_CineCam"));
		CameraMesh = EditorCameraMesh.Object;
	}

	static ConstructorHelpers::FObjectFinder<UStaticMesh> PlaneMesh(TEXT("/Engine/ArtTools/RenderToTexture/Meshes/S_1_Unit_Plane.S_1_Unit_Plane"));
	FocusPlaneVisualizationMesh = PlaneMesh.Object;

	static ConstructorHelpers::FObjectFinder<UMaterial> PlaneMat(TEXT("/Engine/EngineDebugMaterials/M_SimpleUnlitTranslucent.M_SimpleUnlitTranslucent"));
	FocusPlaneVisualizationMaterial = PlaneMat.Object;
#endif

	CustomNearClippingPlane = GNearClippingPlane;
	bOverride_CustomNearClippingPlane = false;
}

void UCineCameraComponent::Serialize(FArchive& Ar)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	
	Ar.UsingCustomVersion(FCineCameraObjectVersion::GUID);
	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);

	Super::Serialize(Ar);

	if (Ar.IsLoading() && Ar.CustomVer(FReleaseObjectVersion::GUID) < FReleaseObjectVersion::DeprecateFilmbackSettings)
	{
		bool bUpgradeFilmback = true;
		if (Ar.CustomVer(FCineCameraObjectVersion::GUID) == FCineCameraObjectVersion::ChangeDefaultFilmbackToDigitalFilm)
		{
			UCineCameraComponent* Template = Cast<UCineCameraComponent>(GetArchetype());
			if (Template)
			{
				TArray<FNamedFilmbackPreset> const& Presets = UCineCameraSettings::GetFilmbackPresets();
				int32 const NumPresets = Presets.Num();
				for (int32 PresetIdx = 0; PresetIdx < NumPresets; ++PresetIdx)
				{
					FNamedFilmbackPreset const& P = Presets[PresetIdx];

					// ChangeDefaultFilmbackToDigitalFilm was pre 4.24, but post 4.23. In that case, the filmback settings would have been DSLR 
					// and RecalcDerivedData would not have been called yet, which equates to SensorAspectRatio being left at 1.33f. This isn't
					// ideal for detecting this case, but it's the best notion of whether upgrading this film back should be skipped and get its 
					// values from the default template object, which is the new Digital Film default.
					if (P.FilmbackSettings == FilmbackSettings_DEPRECATED && FilmbackSettings_DEPRECATED.SensorAspectRatio == 1.33f)
					{
						if (P.Name == Template->DefaultFilmbackPresetName_DEPRECATED)
						{
							bUpgradeFilmback = false;
							break;
						}
					}
				}
			}
		}

		if (bUpgradeFilmback)
		{
			Filmback = FilmbackSettings_DEPRECATED;
		}
	}
	
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UCineCameraComponent::PostInitProperties()
{
	Super::PostInitProperties();

	RecalcDerivedData();
}

void UCineCameraComponent::PostLoad()
{
	Super::PostLoad();

	if (FocusSettings.FocusMethod >= ECameraFocusMethod::MAX )
	{
		FocusSettings.FocusMethod = ECameraFocusMethod::DoNotOverride;
	}

	RecalcDerivedData();
	bResetInterpolation = true;
}

static const FColor DebugFocusPointSolidColor(102, 26, 204, 153);		// purple
static const FColor DebugFocusPointOutlineColor = FColor::Black;

void UCineCameraComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
#if WITH_EDITORONLY_DATA
	// make sure drawing is set up
	if (FocusSettings.bDrawDebugFocusPlane)
	{
		if (DebugFocusPlaneComponent == nullptr)
		{
			CreateDebugFocusPlane();
		}

		UpdateDebugFocusPlane();
	}
	else
	{
		if (DebugFocusPlaneComponent != nullptr)
		{
			DestroyDebugFocusPlane();
		}
	}
#endif

#if ENABLE_DRAW_DEBUG
	if (FocusSettings.TrackingFocusSettings.bDrawDebugTrackingFocusPoint)
	{
		AActor const* const TrackedActor = FocusSettings.TrackingFocusSettings.ActorToTrack.Get();

		FVector FocusPoint;
		if (TrackedActor)
		{
			FTransform const BaseTransform = TrackedActor->GetActorTransform();
			FocusPoint = BaseTransform.TransformPosition(FocusSettings.TrackingFocusSettings.RelativeOffset);
		}
		else
		{
			FocusPoint = FocusSettings.TrackingFocusSettings.RelativeOffset;
		}

		::DrawDebugSolidBox(GetWorld(), FocusPoint, FVector(12.f), DebugFocusPointSolidColor);
		::DrawDebugBox(GetWorld(), FocusPoint, FVector(12.f), DebugFocusPointOutlineColor);
	}
#endif // ENABLE_DRAW_DEBUG

	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
}

#if WITH_EDITORONLY_DATA

void UCineCameraComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName NAME_MinFocalLength = GET_MEMBER_NAME_CHECKED(FCameraLensSettings, MinFocalLength);
	static const FName NAME_MaxFocalLength = GET_MEMBER_NAME_CHECKED(FCameraLensSettings, MaxFocalLength);

	const FName PropertyChangedName = PropertyChangedEvent.GetPropertyName();

	// If the user changed one of these 2 properties, leave the one that they changed alone, and 
	// re-adjust the other one.
	if (PropertyChangedName == NAME_MinFocalLength)
	{
		LensSettings.MaxFocalLength = FMath::Max(LensSettings.MinFocalLength, LensSettings.MaxFocalLength);
	}
	else if (PropertyChangedName == NAME_MaxFocalLength)
	{
		LensSettings.MinFocalLength = FMath::Min(LensSettings.MinFocalLength, LensSettings.MaxFocalLength);
	}

	// Recalculate everything based on any new values.
	RecalcDerivedData();

	// handle debug focus plane
	if (FocusSettings.bDrawDebugFocusPlane && (DebugFocusPlaneComponent == nullptr))
	{
		CreateDebugFocusPlane();
	}
	else if ((FocusSettings.bDrawDebugFocusPlane == false) && (DebugFocusPlaneComponent != nullptr))
	{
		DestroyDebugFocusPlane();
	}

	// set focus plane color in case that's what changed
	if (DebugFocusPlaneMID)
	{
		DebugFocusPlaneMID->SetVectorParameterValue(FName("Color"), FocusSettings.DebugFocusPlaneColor.ReinterpretAsLinear());
	}

	// reset interpolation if the user changes anything
	bResetInterpolation = true;

	UpdateDebugFocusPlane();

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UCineCameraComponent::ResetProxyMeshTransform()
{
	if (ProxyMeshComponent)
	{
		// CineCam mesh is offset 90deg yaw
		ProxyMeshComponent->SetRelativeRotation(FRotator(0.f, 90.f, 0.f));
		ProxyMeshComponent->SetRelativeLocation(FVector(-46.f, 0, -24.f));
	}
}

#endif	// WITH_EDITORONLY_DATA

void UCineCameraComponent::SetFieldOfView(float InFieldOfView)
{
	Super::SetFieldOfView(InFieldOfView);

	float CropedSensorWidth = Filmback.SensorWidth * LensSettings.SqueezeFactor;
	if (CropSettings.AspectRatio > 0.0f)
	{
		float DesqueezeAspectRatio = Filmback.SensorWidth * LensSettings.SqueezeFactor / Filmback.SensorHeight;
		if (CropSettings.AspectRatio < DesqueezeAspectRatio)
		{
			CropedSensorWidth *= CropSettings.AspectRatio / DesqueezeAspectRatio;
		}
	}

	CurrentFocalLength = (CropedSensorWidth / 2.f) / FMath::Tan(FMath::DegreesToRadians(InFieldOfView / 2.f));
}

void UCineCameraComponent::SetCurrentFocalLength(float InFocalLength)
{
	CurrentFocalLength = InFocalLength;
	RecalcDerivedData();
}

float UCineCameraComponent::GetHorizontalFieldOfView() const
{
	return GetHorizontalFieldOfViewInternal(/* bIncludeOverscan =*/ true);
}

float UCineCameraComponent::GetHorizontalFieldOfViewInternal(bool bIncludeOverscan) const
{
	if (CurrentFocalLength > 0.f)
	{
		float CropedSensorWidth = Filmback.SensorWidth * LensSettings.SqueezeFactor;
		if (CropSettings.AspectRatio > 0.0f)
		{
			float DesqueezeAspectRatio = Filmback.SensorWidth * LensSettings.SqueezeFactor / Filmback.SensorHeight;
			if (CropSettings.AspectRatio < DesqueezeAspectRatio)
			{
				CropedSensorWidth *= CropSettings.AspectRatio / DesqueezeAspectRatio;
			}
		}

		const float OverscanScalar = bIncludeOverscan ? (1.0f + Overscan) * 0.5f * (AsymmetricOverscan.X + AsymmetricOverscan.Y + 2.0f) : 1.0f;
		return FMath::RadiansToDegrees(2.f * FMath::Atan(CropedSensorWidth * OverscanScalar / (2.f * CurrentFocalLength)));
	}

	return 0.f;
}

float UCineCameraComponent::GetVerticalFieldOfView() const
{
	return GetVerticalFieldOfViewInternal(/* bIncludeOverscan =*/ true);
}

float UCineCameraComponent::GetVerticalFieldOfViewInternal(bool bIncludeOverscan) const
{
	if (CurrentFocalLength > 0.f)
	{
		float CropedSensorHeight = Filmback.SensorHeight;
		if (CropSettings.AspectRatio > 0.0f)
		{
			float DesqueezeAspectRatio = Filmback.SensorWidth * LensSettings.SqueezeFactor / Filmback.SensorHeight;
			if (DesqueezeAspectRatio < CropSettings.AspectRatio)
			{
				CropedSensorHeight *= DesqueezeAspectRatio / CropSettings.AspectRatio;
			}
		}

		const float OverscanScalar = bIncludeOverscan ? (1.0f + Overscan) * 0.5f * (AsymmetricOverscan.Z + AsymmetricOverscan.W + 2.0f) : 1.0f;
		return FMath::RadiansToDegrees(2.f * FMath::Atan(CropedSensorHeight * OverscanScalar / (2.f * CurrentFocalLength)));
	}

	return 0.f;
}

float UCineCameraComponent::GetHorizontalProjectionOffset() const
{
	float CroppedSensorWidth = Filmback.SensorWidth * LensSettings.SqueezeFactor;
	if (CropSettings.AspectRatio > 0.0f)
	{
		float DesqueezeAspectRatio = Filmback.SensorWidth * LensSettings.SqueezeFactor / Filmback.SensorHeight;
		if (CropSettings.AspectRatio < DesqueezeAspectRatio)
		{
			CroppedSensorWidth *= CropSettings.AspectRatio / DesqueezeAspectRatio;
		}
	}

	const float OverscanScalar = (1.0f + Overscan) * 0.5f * (AsymmetricOverscan.X + AsymmetricOverscan.Y + 2.0f);
	const float AsymmetricOverscanOffset = (AsymmetricOverscan.Y - AsymmetricOverscan.X) / (AsymmetricOverscan.X + AsymmetricOverscan.Y + 2.0f);
	return 2.0f * Filmback.SensorHorizontalOffset / (CroppedSensorWidth * OverscanScalar) + AsymmetricOverscanOffset;
}

float UCineCameraComponent::GetVerticalProjectionOffset() const
{
	float CroppedSensorHeight = Filmback.SensorHeight;
	if (CropSettings.AspectRatio > 0.0f)
	{
		float DesqueezeAspectRatio = Filmback.SensorWidth * LensSettings.SqueezeFactor / Filmback.SensorHeight;
		if (DesqueezeAspectRatio < CropSettings.AspectRatio)
		{
			CroppedSensorHeight *= DesqueezeAspectRatio / CropSettings.AspectRatio;
		}
	}

	const float OverscanScalar = (1.0f + Overscan) * 0.5f * (AsymmetricOverscan.Z + AsymmetricOverscan.W + 2.0f);
	const float AsymmetricOverscanOffset = (AsymmetricOverscan.Z - AsymmetricOverscan.W) / (AsymmetricOverscan.Z + AsymmetricOverscan.W + 2.0f);
	return 2.0f * Filmback.SensorVerticalOffset / (CroppedSensorHeight * OverscanScalar) + AsymmetricOverscanOffset;
}

FString UCineCameraComponent::GetFilmbackPresetName() const
{
	TArray<FNamedFilmbackPreset> const& Presets = UCineCameraSettings::GetFilmbackPresets();
	int32 const NumPresets = Presets.Num();
	for (int32 PresetIdx = 0; PresetIdx < NumPresets; ++PresetIdx)
	{
		FNamedFilmbackPreset const& P = Presets[PresetIdx];
		if (P.FilmbackSettings == Filmback)
		{
			return P.Name;
		}
	}

	return FString();
}

FString UCineCameraComponent::GetDefaultFilmbackPresetName() const
{
	// Try to get the default from the CineCameraSettings object but fallback to an empty string on failure
	if (const UCineCameraSettings* CineCameraSettings = GetDefault<UCineCameraSettings>())
	{
		return CineCameraSettings->DefaultFilmbackPreset;
	}
	return FString();
}

void UCineCameraComponent::SetFilmbackPresetByName(const FString& InPresetName)
{
	SetFilmbackPresetByNameInternal(InPresetName, Filmback);
	// Explicitely call RecalcDerivedData() when invoked via Blueprint, since no other method (incl. PostEditChangeProperty) will trigger
	RecalcDerivedData();
}

void UCineCameraComponent::SetFilmbackPresetByNameInternal(const FString& InPresetName, FCameraFilmbackSettings& InOutFilmbackSettings)
{
	TArray<FNamedFilmbackPreset> const& Presets = UCineCameraSettings::GetFilmbackPresets();
	int32 const NumPresets = Presets.Num();
	for (int32 PresetIdx = 0; PresetIdx < NumPresets; ++PresetIdx)
	{
		FNamedFilmbackPreset const& P = Presets[PresetIdx];
		if (P.Name == InPresetName)
		{
			InOutFilmbackSettings = P.FilmbackSettings;
			break;
		}
	}
}

FString UCineCameraComponent::GetLensPresetName() const
{
	TArray<FNamedLensPreset> const& Presets = UCineCameraSettings::GetLensPresets();
	int32 const NumPresets = Presets.Num();
	for (int32 PresetIdx = 0; PresetIdx < NumPresets; ++PresetIdx)
	{
		FNamedLensPreset const& P = Presets[PresetIdx];
		if (P.LensSettings == LensSettings)
		{
			return P.Name;
		}
	}

	return FString();
}

void UCineCameraComponent::SetLensPresetByName(const FString& InPresetName)
{
	SetLensPresetByNameInternal(InPresetName);
	// Explicitely call RecalcDerivedData() when invoked via Blueprint, since no other method (incl. PostEditChangeProperty) will trigger
	RecalcDerivedData();
}

void UCineCameraComponent::SetLensPresetByNameInternal(const FString& InPresetName)
{
	TArray<FNamedLensPreset> const& Presets = UCineCameraSettings::GetLensPresets();
	int32 const NumPresets = Presets.Num();
	for (int32 PresetIdx = 0; PresetIdx < NumPresets; ++PresetIdx)
	{
		FNamedLensPreset const& P = Presets[PresetIdx];
		if (P.Name == InPresetName)
		{
			LensSettings = P.LensSettings;
			break;
		}
	}
}

FString UCineCameraComponent::GetCropPresetName() const
{
	TArray<FNamedPlateCropPreset> const& Presets = UCineCameraSettings::GetCropPresets();
	int32 const NumPresets = Presets.Num();
	for (int32 PresetIdx = 0; PresetIdx < NumPresets; ++PresetIdx)
	{
		FNamedPlateCropPreset const& P = Presets[PresetIdx];
		if (P.CropSettings == CropSettings)
		{
			return P.Name;
		}
	}

	return FString();
}

void UCineCameraComponent::SetCropPresetByName(const FString& InPresetName)
{
	SetCropPresetByNameInternal(InPresetName);
	// Explicitely call RecalcDerivedData() when invoked via Blueprint, since no other method (incl. PostEditChangeProperty) will trigger
	RecalcDerivedData();
}

void UCineCameraComponent::SetCropPresetByNameInternal(const FString& InPresetName)
{
	TArray<FNamedPlateCropPreset> const& Presets = UCineCameraSettings::GetCropPresets();
	int32 const NumPresets = Presets.Num();
	for (int32 PresetIdx = 0; PresetIdx < NumPresets; ++PresetIdx)
	{
		FNamedPlateCropPreset const& P = Presets[PresetIdx];
		if (P.Name == InPresetName)
		{
			CropSettings = P.CropSettings;
			break;
		}
	}
}

float UCineCameraComponent::GetWorldToMetersScale() const
{
	UWorld const* const World = GetWorld();
	AWorldSettings const* const WorldSettings = World ? World->GetWorldSettings() : nullptr;
	return WorldSettings ? WorldSettings->WorldToMeters : 100.f;
}

// static
TArray<FNamedFilmbackPreset> UCineCameraComponent::GetFilmbackPresetsCopy()
{
	return UCineCameraSettings::GetFilmbackPresets();
}

// static
TArray<FNamedLensPreset> UCineCameraComponent::GetLensPresetsCopy()
{
	return UCineCameraSettings::GetLensPresets();
}

// static
TArray<FNamedFilmbackPreset> const& UCineCameraComponent::GetFilmbackPresets()
{
	return UCineCameraSettings::GetFilmbackPresets();
}

// static
TArray<FNamedLensPreset> const& UCineCameraComponent::GetLensPresets()
{
	return UCineCameraSettings::GetLensPresets();
}

void UCineCameraComponent::RecalcDerivedData()
{
	// validate incorrect values
	LensSettings.MaxFocalLength = FMath::Max(LensSettings.MinFocalLength, LensSettings.MaxFocalLength);
	
	// respect physical limits of the (simulated) hardware
	CurrentFocalLength = FMath::Clamp(CurrentFocalLength, LensSettings.MinFocalLength, LensSettings.MaxFocalLength);
	CurrentAperture = FMath::Clamp(CurrentAperture, LensSettings.MinFStop, LensSettings.MaxFStop);

	float const MinFocusDistInWorldUnits = LensSettings.MinimumFocusDistance * (GetWorldToMetersScale() / 1000.f);	// convert mm to uu
	FocusSettings.ManualFocusDistance = FMath::Max(FocusSettings.ManualFocusDistance, MinFocusDistInWorldUnits);

	FieldOfView = GetHorizontalFieldOfViewInternal(/*bIncludeOverscan =*/ false);
	Filmback.RecalcSensorAspectRatio();
	AspectRatio = (CropSettings.AspectRatio > 0.0f ? CropSettings.AspectRatio : Filmback.SensorAspectRatio * LensSettings.SqueezeFactor);

#if WITH_EDITORONLY_DATA
	CurrentHorizontalFOV = FieldOfView;			// informational variable only, for editor users
#endif
}

/// @endcond

float UCineCameraComponent::GetDesiredFocusDistance(const FVector& InLocation) const
{
	float DesiredFocusDistance = 0.f;

	// get focus distance
	switch (FocusSettings.FocusMethod)
	{
	case ECameraFocusMethod::Manual:
		DesiredFocusDistance = FocusSettings.ManualFocusDistance;
		break;

	case ECameraFocusMethod::Tracking:
		{
			AActor const* const TrackedActor = FocusSettings.TrackingFocusSettings.ActorToTrack.Get();

			FVector FocusPoint;
			if (TrackedActor)
			{
				FTransform const BaseTransform = TrackedActor->GetActorTransform();
				FocusPoint = BaseTransform.TransformPosition(FocusSettings.TrackingFocusSettings.RelativeOffset);
			}
			else
			{
				FocusPoint = FocusSettings.TrackingFocusSettings.RelativeOffset;
			}

			const bool bBehindCamera = FVector::DotProduct(GetForwardVector(), FocusPoint - InLocation) < 0;

			if (bBehindCamera)
			{
				DesiredFocusDistance = 0;
			}
			else
			{
				FVector OutClosestPoint;
				FMath::PointDistToLine(FocusPoint, GetForwardVector(), InLocation, OutClosestPoint);
				DesiredFocusDistance = (OutClosestPoint - InLocation).Size();
			}
		}
		break;
	}
	
	// add in the adjustment offset
	DesiredFocusDistance += FocusSettings.FocusOffset;

	return DesiredFocusDistance;
}

void UCineCameraComponent::GetCameraView(float DeltaTime, FMinimalViewInfo& DesiredView)
{
	RecalcDerivedData();

	Super::GetCameraView(DeltaTime, DesiredView);

	UpdateCameraLens(DeltaTime, DesiredView);

	DesiredView.PerspectiveNearClipPlane = bOverride_CustomNearClippingPlane ? CustomNearClippingPlane : -1.0f;

	DesiredView.OffCenterProjectionOffset.X = GetHorizontalProjectionOffset();
	DesiredView.OffCenterProjectionOffset.Y = GetVerticalProjectionOffset();
	
	bResetInterpolation = false;
}

#if WITH_EDITOR
FText UCineCameraComponent::GetFilmbackText() const
{
	const float SensorWidth = Filmback.SensorWidth;
	const float SensorHeight = Filmback.SensorHeight;

	// Search presets for one that matches
	const FNamedFilmbackPreset* Preset = UCineCameraSettings::GetFilmbackPresets().FindByPredicate([&](const FNamedFilmbackPreset& InPreset) {
		return InPreset.FilmbackSettings.SensorWidth == SensorWidth && InPreset.FilmbackSettings.SensorHeight == SensorHeight;
	});

	if (Preset)
	{
		return FText::Format(
			LOCTEXT("PresetFormat","FilmbackPreset: {0} | Zoom: {1}mm | Av: {2} | Squeeze: {3}"),
			Preset->DisplayName.IsEmpty() ? FText::FromString(Preset->Name) : Preset->DisplayName,
			FText::AsNumber(CurrentFocalLength),
			FText::AsNumber(CurrentAperture),
			FText::AsNumber(LensSettings.SqueezeFactor)
		);
	}
	else
	{
		FNumberFormattingOptions Opts = FNumberFormattingOptions().SetMaximumFractionalDigits(1);
		return FText::Format(
			LOCTEXT("CustomFilmbackFormat", "Custom ({0}mm x {1}mm) | Zoom: {2}mm | Av: {3} | Squeeze: {4}"),
			FText::AsNumber(SensorWidth, &Opts),
			FText::AsNumber(SensorHeight, &Opts),
			FText::AsNumber(CurrentFocalLength),
			FText::AsNumber(CurrentAperture),
			FText::AsNumber(LensSettings.SqueezeFactor)
		);
	}
}
#endif

void UCineCameraComponent::SetFilmback(const FCameraFilmbackSettings& NewFilmback)
{
	Filmback = NewFilmback;
	RecalcDerivedData();
}

void UCineCameraComponent::SetLensSettings(const FCameraLensSettings& NewLensSettings)
{
	LensSettings = NewLensSettings;
	RecalcDerivedData();
}

void UCineCameraComponent::SetFocusSettings(const FCameraFocusSettings& NewFocusSettings)
{
	FocusSettings = NewFocusSettings;
	RecalcDerivedData();
}

void UCineCameraComponent::SetCropSettings(const FPlateCropSettings& NewCropSettings)
{
	CropSettings = NewCropSettings;
	RecalcDerivedData();
}

void UCineCameraComponent::SetCurrentAperture(const float NewCurrentAperture)
{
	CurrentAperture = NewCurrentAperture;
	RecalcDerivedData();
}

void UCineCameraComponent::SetCustomNearClippingPlane(const float NewCustomNearClippingPlane)
{
	CustomNearClippingPlane = NewCustomNearClippingPlane;
}

#if WITH_EDITORONLY_DATA
void UCineCameraComponent::UpdateDebugFocusPlane()
{
	if (FocusPlaneVisualizationMesh && DebugFocusPlaneComponent)
	{
		FVector const CamLocation = GetComponentTransform().GetLocation();
		FVector const CamDir = GetComponentTransform().GetRotation().Vector();

		UWorld const* const World = GetWorld();
		float const FocusDistance = (World && World->IsGameWorld()) ? CurrentFocusDistance : GetDesiredFocusDistance(CamLocation);		// in editor, use desired focus distance directly, no interp
		FVector const FocusPoint = GetComponentTransform().GetLocation() + CamDir * FocusDistance;

		DebugFocusPlaneComponent->SetWorldLocation(FocusPoint);
	}
}
#endif

void UCineCameraComponent::UpdateCameraLens(float DeltaTime, FMinimalViewInfo& DesiredView)
{
	if (ExposureMethod == ECameraExposureMethod::DoNotOverride &&
			FocusSettings.FocusMethod == ECameraFocusMethod::DoNotOverride)
	{
		// We want neither dynamic exposure nor focus effects.
		DesiredView.PostProcessSettings.bOverride_DepthOfFieldFstop = false;
	}
	else
	{
		// We want dynamic exposure or focus effects, maybe both.
		DesiredView.PostProcessSettings.bOverride_DepthOfFieldFstop = true;
		DesiredView.PostProcessSettings.DepthOfFieldFstop = CurrentAperture;
	}

	if (FocusSettings.FocusMethod == ECameraFocusMethod::DoNotOverride)
	{
		DesiredView.PostProcessSettings.bOverride_DepthOfFieldMinFstop = false;
		DesiredView.PostProcessSettings.bOverride_DepthOfFieldBladeCount = false;
		DesiredView.PostProcessSettings.bOverride_DepthOfFieldFocalDistance = false;
		DesiredView.PostProcessSettings.bOverride_DepthOfFieldSensorWidth = false;
		DesiredView.PostProcessSettings.bOverride_DepthOfFieldSqueezeFactor = false;
	}
	else if (FocusSettings.FocusMethod == ECameraFocusMethod::Disable)
	{
		// There might be a post process volume that is enabled with depth of field settings, override it and disable depth of field by setting the distance to 0
		DesiredView.PostProcessSettings.bOverride_DepthOfFieldFocalDistance = true;
		DesiredView.PostProcessSettings.DepthOfFieldFocalDistance = 0.f;
	}
	else
	{
		// Update focus/DoF
		DesiredView.PostProcessBlendWeight = PostProcessBlendWeight;

		DesiredView.PostProcessSettings.bOverride_DepthOfFieldMinFstop = true;
		DesiredView.PostProcessSettings.DepthOfFieldMinFstop = LensSettings.MinFStop;

		DesiredView.PostProcessSettings.bOverride_DepthOfFieldBladeCount = true;
		DesiredView.PostProcessSettings.DepthOfFieldBladeCount = LensSettings.DiaphragmBladeCount;

		CurrentFocusDistance = GetDesiredFocusDistance(DesiredView.Location);

		// clamp to min focus distance
		float const MinFocusDistInWorldUnits = LensSettings.MinimumFocusDistance * (GetWorldToMetersScale() / 1000.f);	// convert mm to uu
		CurrentFocusDistance = FMath::Max(CurrentFocusDistance, MinFocusDistInWorldUnits);

		// smoothing, if desired
		if (FocusSettings.bSmoothFocusChanges)
		{
			if (bResetInterpolation == false)
			{
				CurrentFocusDistance = FMath::FInterpTo(LastFocusDistance, CurrentFocusDistance, DeltaTime, FocusSettings.FocusSmoothingInterpSpeed);
			}
		}
		LastFocusDistance = CurrentFocusDistance;

		DesiredView.PostProcessSettings.bOverride_DepthOfFieldFocalDistance = true;
		DesiredView.PostProcessSettings.DepthOfFieldFocalDistance = CurrentFocusDistance;

		const float OverscanScalar = (1.0f + Overscan) * 0.5f * (AsymmetricOverscan.X + AsymmetricOverscan.Y + 2.0f);
		DesiredView.PostProcessSettings.bOverride_DepthOfFieldSensorWidth = true;
		DesiredView.PostProcessSettings.DepthOfFieldSensorWidth = Filmback.SensorWidth * OverscanScalar;

		const float AsymmetricSqueezeFactor = bCropOverscan ? (AsymmetricOverscan.X + AsymmetricOverscan.Y + 2.0f) / (AsymmetricOverscan.Z + AsymmetricOverscan.W + 2.0f) : 1.0f;
		DesiredView.PostProcessSettings.bOverride_DepthOfFieldSqueezeFactor = true;
		DesiredView.PostProcessSettings.DepthOfFieldSqueezeFactor = LensSettings.SqueezeFactor * AsymmetricSqueezeFactor;
		DesiredView.PostProcessSettings.bOverride_DepthOfFieldAspectRatioScalar = true;
		DesiredView.PostProcessSettings.DepthOfFieldAspectRatioScalar = AsymmetricSqueezeFactor;
	}
}

void UCineCameraComponent::NotifyCameraCut()
{
	Super::NotifyCameraCut();

	// reset any interpolations
	bResetInterpolation = true;
}

#if WITH_EDITORONLY_DATA
void UCineCameraComponent::CreateDebugFocusPlane()
{
	if (AActor* const MyOwner = GetOwner())
	{
		if (DebugFocusPlaneComponent == nullptr)
		{
			DebugFocusPlaneComponent = NewObject<UStaticMeshComponent>(MyOwner, NAME_None, RF_Transient | RF_Transactional | RF_TextExportTransient);
			DebugFocusPlaneComponent->SetupAttachment(this);
			DebugFocusPlaneComponent->SetIsVisualizationComponent(true);
			DebugFocusPlaneComponent->SetStaticMesh(FocusPlaneVisualizationMesh);
			DebugFocusPlaneComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
			DebugFocusPlaneComponent->bHiddenInGame = false;
			DebugFocusPlaneComponent->CastShadow = false;
			DebugFocusPlaneComponent->CreationMethod = CreationMethod;
			DebugFocusPlaneComponent->bSelectable = false;
			DebugFocusPlaneComponent->SetIgnoreBoundsForEditorFocus(true);

			DebugFocusPlaneComponent->SetRelativeScale3D_Direct(FVector(10000.f, 10000.f, 1.f));
			DebugFocusPlaneComponent->SetRelativeRotation_Direct(FRotator(90.f, 0.f, 0.f));

			DebugFocusPlaneComponent->RegisterComponentWithWorld(GetWorld());

			DebugFocusPlaneMID = DebugFocusPlaneComponent->CreateAndSetMaterialInstanceDynamicFromMaterial(0, FocusPlaneVisualizationMaterial);
			if (DebugFocusPlaneMID)
			{
				DebugFocusPlaneMID->SetVectorParameterValue(FName("Color"), FocusSettings.DebugFocusPlaneColor.ReinterpretAsLinear());
			}
		}
	}
}

void UCineCameraComponent::DestroyDebugFocusPlane()
{
	if (DebugFocusPlaneComponent)
	{
		DebugFocusPlaneComponent->SetVisibility(false);
		DebugFocusPlaneComponent = nullptr;

		DebugFocusPlaneMID = nullptr;
	}
}
#endif

void UCineCameraComponent::OnRegister()
{
	Super::OnRegister();

#if WITH_EDITORONLY_DATA
	ResetProxyMeshTransform();
#endif
}

#if WITH_EDITOR
void UCineCameraComponent::OnComponentDestroyed(bool bDestroyingHierarchy)
{
	Super::OnComponentDestroyed(bDestroyingHierarchy);

	if (DebugFocusPlaneComponent)
	{
		DebugFocusPlaneComponent->DestroyComponent();
	}
}
#endif

#undef LOCTEXT_NAMESPACE

