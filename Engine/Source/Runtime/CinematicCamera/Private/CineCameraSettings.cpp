// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineCameraSettings.h"

#include "CineCameraComponent.h"
#include "Misc/ConfigCacheIni.h"

#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CineCameraSettings)

#define LOCTEXT_NAMESPACE "CineCameraSettings"

const FString UCineCameraSettings::CineCameraConfigSection = TEXT("/Script/CinematicCamera.CineCameraComponent");

void UCineCameraSettings::PostInitProperties()
{
	Super::PostInitProperties();

// The Fixup Notifications should only be displayed in Editor
#if WITH_EDITOR
	if (GConfig && GConfig->DoesSectionExist(*CineCameraConfigSection, GEngineIni))
	{
		FNotificationInfo NotificationInfo(LOCTEXT("UpdateSettingsNotification", "CineCamera Settings were found in an old config location. Do you want to attempt to automatically merge them to the new location?"));
		NotificationInfo.FadeOutDuration = 0.5f;
		NotificationInfo.bFireAndForget = false;
		NotificationInfo.bUseSuccessFailIcons = true;

		const FNotificationButtonInfo OkButton(LOCTEXT("MergeButtonText", "Merge"), FText::GetEmpty(), FSimpleDelegate::CreateUObject(this, &UCineCameraSettings::CopyOldConfigSettings), SNotificationItem::ECompletionState::CS_None);
		const FNotificationButtonInfo CancelButton(LOCTEXT("CancelButtonText", "Cancel"), FText::GetEmpty(), FSimpleDelegate::CreateUObject(this, &UCineCameraSettings::CloseNotification), SNotificationItem::ECompletionState::CS_None);

		NotificationInfo.ButtonDetails.Add(OkButton);
		NotificationInfo.ButtonDetails.Add(CancelButton);

		Notification = FSlateNotificationManager::Get().AddNotification(NotificationInfo);
	}
#endif
	RecalcSensorAspectRatios();
}

#if WITH_EDITOR
void UCineCameraSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UCineCameraSettings, FilmbackPresets))
	{
		RecalcSensorAspectRatios();
	}
	
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

void UCineCameraSettings::SetDefaultLensPresetName(const FString InDefaultLensPresetName)
{
	DefaultLensPresetName = InDefaultLensPresetName;
	SaveConfig();
}

void UCineCameraSettings::SetDefaultLensFocalLength(const float InDefaultLensFocalLength)
{
	DefaultLensFocalLength = InDefaultLensFocalLength;
	SaveConfig();
}

void UCineCameraSettings::SetDefaultLensFStop(const float InDefaultLensFStop)
{
	DefaultLensFStop = InDefaultLensFStop;
	SaveConfig();
}

void UCineCameraSettings::SetLensPresets(const TArray<FNamedLensPreset>& InLensPresets)
{
	LensPresets = InLensPresets;
	SaveConfig();
}

void UCineCameraSettings::SetDefaultFilmbackPreset(const FString InDefaultFilmbackPreset)
{
	DefaultFilmbackPreset = InDefaultFilmbackPreset;
	SaveConfig();
}

void UCineCameraSettings::SetFilmbackPresets(const TArray<FNamedFilmbackPreset>& InFilmbackPresets)
{
	FilmbackPresets = InFilmbackPresets;
	RecalcSensorAspectRatios();
	SaveConfig();
}

void UCineCameraSettings::SetDefaultCropPresetName(const FString InDefaultCropPresetName)
{
	DefaultCropPresetName = InDefaultCropPresetName;
	SaveConfig();
}

void UCineCameraSettings::SetCropPresets(const TArray<FNamedPlateCropPreset>& InCropPresets)
{
	CropPresets = InCropPresets;
	SaveConfig();
}

TArray<FNamedLensPreset> const& UCineCameraSettings::GetLensPresets()
{
	return GetDefault<UCineCameraSettings>()->LensPresets;
}

TArray<FNamedFilmbackPreset> const& UCineCameraSettings::GetFilmbackPresets()
{
	return GetDefault<UCineCameraSettings>()->FilmbackPresets;
}

TArray<FNamedPlateCropPreset> const& UCineCameraSettings::GetCropPresets()
{
	return GetDefault<UCineCameraSettings>()->CropPresets;
}

bool UCineCameraSettings::GetLensPresetByName(const FString PresetName, FCameraLensSettings& LensSettings)
{
	FNamedLensPreset* NamedLensPreset = LensPresets.FindByPredicate([PresetName](const FNamedLensPreset& Preset)
	{
		return Preset.Name == PresetName;
	});

	LensSettings = NamedLensPreset ? NamedLensPreset->LensSettings : FCameraLensSettings();
	return NamedLensPreset != nullptr;
}

bool UCineCameraSettings::GetFilmbackPresetByName(const FString PresetName, FCameraFilmbackSettings& FilmbackSettings)
{
	FNamedFilmbackPreset* NamedFilmbackPreset = FilmbackPresets.FindByPredicate([PresetName](const FNamedFilmbackPreset& Preset)
	{
		return Preset.Name == PresetName;
	});

	FilmbackSettings = NamedFilmbackPreset ? NamedFilmbackPreset->FilmbackSettings : FCameraFilmbackSettings();
	return NamedFilmbackPreset != nullptr;
}

bool UCineCameraSettings::GetCropPresetByName(const FString PresetName, FPlateCropSettings& CropSettings)
{
	FNamedPlateCropPreset* NamedCropPreset = CropPresets.FindByPredicate([PresetName](const FNamedPlateCropPreset& Preset)
	{
		return Preset.Name == PresetName;
	});

	CropSettings = NamedCropPreset ? NamedCropPreset->CropSettings : FPlateCropSettings();
	return NamedCropPreset != nullptr;
}

UCineCameraSettings* UCineCameraSettings::GetCineCameraSettings()
{
	return GetMutableDefault<UCineCameraSettings>();
}

TArray<FString> UCineCameraSettings::GetLensPresetNames() const
{
	TArray<FString> LensPresetNames;
	for (const FNamedLensPreset& LensPreset : LensPresets)
	{
		LensPresetNames.Emplace(LensPreset.Name);
	}

	return LensPresetNames;
}

TArray<FString> UCineCameraSettings::GetFilmbackPresetNames() const
{	
	TArray<FString> FilmbackPresetNames;
	for (const FNamedFilmbackPreset& FilmbackPreset : FilmbackPresets)
	{
		FilmbackPresetNames.Emplace(FilmbackPreset.Name);
	}
	
	return FilmbackPresetNames;
}

TArray<FString> UCineCameraSettings::GetCropPresetNames() const
{
	TArray<FString> CropPresetNames;
	for (const FNamedPlateCropPreset& CropPreset : CropPresets)
	{
		CropPresetNames.Emplace(CropPreset.Name);
	}

	return CropPresetNames;
}

void UCineCameraSettings::CloseNotification()
{
	if (Notification)
	{
		Notification->ExpireAndFadeout();
		Notification = nullptr;
	}
	
	FNotificationInfo NotificationInfo(LOCTEXT("SettingsMergeCancelled", "Please manually fix up and remove the values from the CineCameraComponent section"));
	NotificationInfo.ExpireDuration = 8.0f;
	NotificationInfo.bFireAndForget = true;

	FSlateNotificationManager::Get().AddNotification(NotificationInfo);
}

void UCineCameraSettings::RecalcSensorAspectRatios()
{
	for (FNamedFilmbackPreset& FilmbackPreset : FilmbackPresets)
	{
		FilmbackPreset.FilmbackSettings.RecalcSensorAspectRatio();	
	}
}

void UCineCameraSettings::CopyOldConfigSettings()
{	
	const FString SettingsConfigSection("/Script/CinematicCamera.CineCameraSettings");

	FString OldDefaultLensPresetName;
	if (GConfig->GetString(*CineCameraConfigSection, TEXT("DefaultLensPresetName"), OldDefaultLensPresetName, GEngineIni))
	{
		GConfig->SetString(*SettingsConfigSection, TEXT("DefaultLensPresetName"), *OldDefaultLensPresetName, GEngineIni);
	}

	float OldDefaultLensFocalLength;
	if (GConfig->GetFloat(*CineCameraConfigSection, TEXT("DefaultLensFocalLength"), OldDefaultLensFocalLength, GEngineIni))
	{
		GConfig->SetFloat(*SettingsConfigSection, TEXT("DefaultLensFocalLength"), OldDefaultLensFocalLength, GEngineIni);
	}

	float OldDefaultLensFStop;
	if (GConfig->GetFloat(*CineCameraConfigSection, TEXT("DefaultLensFStop"), OldDefaultLensFStop, GEngineIni))
	{
		GConfig->SetFloat(*SettingsConfigSection, TEXT("DefaultLensFStop"), OldDefaultLensFStop, GEngineIni);
	}

	TArray<FString> OldLensPresets;
	if (GConfig->GetArray(*CineCameraConfigSection, TEXT("LensPresets"), OldLensPresets, GEngineIni))
	{
		TArray<FString> CurrentLensPresets;
		GConfig->GetArray(*SettingsConfigSection, TEXT("LensPresets"), CurrentLensPresets, GEngineIni);
		for (const FString& OldLensPreset : OldLensPresets)
		{
			// If a preset already exists with that name then update it with the old value
			// otherwise add the preset to the list
			FString PresetName, PresetValue;
			OldLensPreset.Split(",", &PresetName, &PresetValue);
			if (FString* MatchingLensPreset = CurrentLensPresets.FindByPredicate([PresetName](const FString& CurrentPreset){ return CurrentPreset.StartsWith(PresetName); }))
			{
				*MatchingLensPreset = OldLensPreset;	
			}
			else
			{
				CurrentLensPresets.Add(OldLensPreset);
			}
		}
		GConfig->SetArray(*SettingsConfigSection, TEXT("LensPresets"), CurrentLensPresets, GEngineIni);
	}

	FString OldDefaultFilmbackPreset;
	if (GConfig->GetString(*CineCameraConfigSection, TEXT("DefaultFilmbackPreset"), OldDefaultFilmbackPreset, GEngineIni))
	{
		GConfig->SetString(*SettingsConfigSection, TEXT("DefaultFilmbackPreset"), *OldDefaultFilmbackPreset, GEngineIni);
	}

	TArray<FString> OldFilmbackPresets;
	if (GConfig->GetArray(*CineCameraConfigSection, TEXT("FilmbackPresets"), OldFilmbackPresets, GEngineIni))
	{
		TArray<FString> CurrentFilmbackPresets;
		GConfig->GetArray(*SettingsConfigSection, TEXT("FilmbackPresets"), CurrentFilmbackPresets, GEngineIni);
		for (const FString& OldFilmbackPreset : OldFilmbackPresets)
		{
			// If a preset already exists with that name then update it with the old value
			// otherwise add the preset to the list
			FString PresetName, PresetValue;
			OldFilmbackPreset.Split(",", &PresetName, &PresetValue);
			if (FString* MatchingPreset = CurrentFilmbackPresets.FindByPredicate([PresetName](const FString& CurrentPreset){ return CurrentPreset.StartsWith(PresetName); }))
			{
				*MatchingPreset = OldFilmbackPreset;
			}
			else
			{
				CurrentFilmbackPresets.Add(OldFilmbackPreset);
			}
		}
		GConfig->SetArray(*SettingsConfigSection, TEXT("FilmbackPresets"), CurrentFilmbackPresets, GEngineIni);
	}

	LoadConfig();
	
	if (Notification)
	{
		Notification->ExpireAndFadeout();
		Notification = nullptr;
	}

	FNotificationInfo NotificationInfo(LOCTEXT("NotifySettingsUpdated", "CineCamera Settings have been successfully merged.\n\nPlease remove the old config values from the CineCameraComponent section"));
	NotificationInfo.ExpireDuration = 8.0f;
	NotificationInfo.bFireAndForget = true;

	FSlateNotificationManager::Get().AddNotification(NotificationInfo);
}

#undef LOCTEXT_NAMESPACE
