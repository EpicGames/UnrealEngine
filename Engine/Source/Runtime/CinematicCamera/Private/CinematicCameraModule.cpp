// Copyright Epic Games, Inc. All Rights Reserved.

#include "CinematicCameraModule.h"
#include "Modules/ModuleManager.h"

#include "CineCameraComponent.h"
#include "MovieSceneTracksComponentTypes.h"


struct FCinematicCameraModule : ICinematicCameraModule
{
	void StartupModule() override
	{
		using namespace UE::MovieScene;

		FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

		TracksComponents->Accessors.Float.Add(
				UCineCameraComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(UCineCameraComponent, CurrentAperture), 
				GetCurrentAperture, SetCurrentAperture);

		TracksComponents->Accessors.Float.Add(
				UCineCameraComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(UCineCameraComponent, CurrentFocalLength), 
				GetCurrentFocalLength, SetCurrentFocalLength);
	}

	static float GetCurrentAperture(const UObject* Object)
	{
		return CastChecked<const UCineCameraComponent>(Object)->CurrentAperture;
	}

	static void SetCurrentAperture(UObject* Object, float InNewAperture)
	{
		CastChecked<UCineCameraComponent>(Object)->SetCurrentAperture(InNewAperture);
	}

	static float GetCurrentFocalLength(const UObject* Object)
	{
		return CastChecked<const UCineCameraComponent>(Object)->CurrentFocalLength;
	}

	static void SetCurrentFocalLength(UObject* Object, float InNewFocalLength)
	{
		CastChecked<UCineCameraComponent>(Object)->SetCurrentFocalLength(InNewFocalLength);
	}
};


IMPLEMENT_MODULE(FCinematicCameraModule, CinematicCamera );
