// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

//#include "Engine/Scene.h"

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "CameraRig_Crane.generated.h"

class UStaticMeshComponent;

/** 
 * A simple rig for simulating crane-like camera movements.
 */
UCLASS(Blueprintable, MinimalAPI)
class ACameraRig_Crane : public AActor
{
	GENERATED_BODY()
	
public:

	// ctor
	CINEMATICCAMERA_API ACameraRig_Crane(const FObjectInitializer& ObjectInitialier);

	CINEMATICCAMERA_API virtual void Tick(float DeltaTime) override;
	CINEMATICCAMERA_API virtual bool ShouldTickIfViewportsOnly() const override;

	/** Controls the pitch of the crane arm. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Crane Controls", meta = (UIMin = "-360", UIMax = "360", Units = deg))
	float CranePitch;
	
	/** Controls the yaw of the crane arm. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Crane Controls", meta = (UIMin = "-360", UIMax = "360", Units = deg))
	float CraneYaw;
	
	/** Controls the length of the crane arm. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Crane Controls", meta = (ClampMin=0, Units = cm))
	float CraneArmLength;

	/** Lock the mount pitch so that an attached camera is locked and pitched in the direction of the crane arm */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Crane Controls")
	bool bLockMountPitch;

	/** Lock the mount yaw so that an attached camera is locked and oriented in the direction of the crane arm */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Crane Controls")
	bool bLockMountYaw;

#if WITH_EDITOR
	CINEMATICCAMERA_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	CINEMATICCAMERA_API virtual void PostEditUndo() override;
#endif
	CINEMATICCAMERA_API virtual class USceneComponent* GetDefaultAttachComponent() const override;
	
private:
#if WITH_EDITORONLY_DATA
	void UpdatePreviewMeshes();
#endif
	void UpdateCraneComponents();

	/** Root component to give the whole actor a transform. */
	UPROPERTY(EditDefaultsOnly, Category = "Crane Components")
	TObjectPtr<USceneComponent> TransformComponent;

	/** Component to control Yaw. */
	UPROPERTY(EditDefaultsOnly, Category = "Crane Components")
	TObjectPtr<USceneComponent> CraneYawControl;

	/** Component to control Pitch. */
	UPROPERTY(EditDefaultsOnly, Category = "Crane Components")
	TObjectPtr<USceneComponent> CranePitchControl;

	/** Component to define the attach point for cameras. */
	UPROPERTY(EditDefaultsOnly, Category = "Crane Components")
	TObjectPtr<USceneComponent> CraneCameraMount;

#if WITH_EDITORONLY_DATA
	/** Preview meshes for visualization */
	UPROPERTY(Transient)
	TObjectPtr<UStaticMeshComponent> PreviewMesh_CraneArm;

	UPROPERTY(Transient)
	TObjectPtr<UStaticMeshComponent> PreviewMesh_CraneBase;

	UPROPERTY(Transient)
	TObjectPtr<UStaticMeshComponent> PreviewMesh_CraneMount;

	UPROPERTY(Transient)
	TObjectPtr<UStaticMeshComponent> PreviewMesh_CraneCounterWeight;
#endif
};
