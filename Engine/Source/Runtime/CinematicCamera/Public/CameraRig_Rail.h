// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "CameraRig_Rail.generated.h"

class USplineComponent;
class USplineMeshComponent;
class UStaticMesh;
class UStaticMeshComponent;

/** 
 * 
 */
UCLASS(Blueprintable, MinimalAPI)
class ACameraRig_Rail : public AActor
{
	GENERATED_BODY()
	
public:
	// ctor
	CINEMATICCAMERA_API ACameraRig_Rail(const FObjectInitializer& ObjectInitialier);
	
	CINEMATICCAMERA_API virtual void Tick(float DeltaTime) override;
	CINEMATICCAMERA_API virtual bool ShouldTickIfViewportsOnly() const override;

	/** Defines current position of the mount point along the rail, in terms of normalized distance from the beginning of the rail. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Rail Controls", meta=(ClampMin="0.0", ClampMax = "1.0"))
	float CurrentPositionOnRail;

	/** Determines whether the orientation of the mount should be in the direction of the rail. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Rail Controls")
	bool bLockOrientationToRail;

#if WITH_EDITORONLY_DATA
	/** Determines whether or not to show the rail mesh preview. */
	UPROPERTY(Transient, EditAnywhere, Category = "Rail Controls")
	bool bShowRailVisualization;

	/** Determines the scale of the rail mesh preview */
	UPROPERTY(Transient, EditAnywhere, Category = "Rail Controls")
	float PreviewMeshScale;
#endif

	CINEMATICCAMERA_API virtual class USceneComponent* GetDefaultAttachComponent() const override;
#if WITH_EDITOR
	CINEMATICCAMERA_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	CINEMATICCAMERA_API virtual void PostEditUndo() override;
	CINEMATICCAMERA_API virtual void PostEditMove(bool bFinished) override;
#endif

	/** Returns the spline component that defines the rail path */
	UFUNCTION(BlueprintPure, Category = "Rail Components")
	USplineComponent* GetRailSplineComponent() { return RailSplineComponent; }
	
protected:
	/** Makes sure all components are arranged properly. Call when something changes that might affect components. */
	CINEMATICCAMERA_API virtual void UpdateRailComponents();
#if WITH_EDITORONLY_DATA
	CINEMATICCAMERA_API void UpdatePreviewMeshes();
#endif
	CINEMATICCAMERA_API USplineMeshComponent* CreateSplinePreviewSegment();

 	/** Root component to give the whole actor a transform. */
	UPROPERTY(EditDefaultsOnly, Category = "Rail Components")
 	TObjectPtr<USceneComponent> TransformComponent;

	/** Spline component to define the rail path. */
	UPROPERTY(EditDefaultsOnly, Category = "Rail Components")
	TObjectPtr<USplineComponent> RailSplineComponent;

	/** Component to define the attach point for cameras. Moves along the rail. */
	UPROPERTY(EditDefaultsOnly, Category = "Rail Components")
	TObjectPtr<USceneComponent> RailCameraMount;

#if WITH_EDITORONLY_DATA
	/** Preview meshes for visualization */
	UPROPERTY(Transient)
	TObjectPtr<USplineMeshComponent> PreviewMesh_Rail;

	UPROPERTY(Transient)
	TArray<TObjectPtr<USplineMeshComponent>> PreviewRailMeshSegments;

	UPROPERTY(Transient)
	TObjectPtr<UStaticMesh> PreviewRailStaticMesh;

	UPROPERTY(Transient)
	TObjectPtr<UStaticMeshComponent> PreviewMesh_Mount;
#endif
};
