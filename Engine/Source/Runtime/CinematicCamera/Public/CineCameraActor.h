// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Camera/CameraActor.h"
#include "CoreMinimal.h"
#include "Math/Rotator.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "CineCameraActor.generated.h"

class AActor;
class UCineCameraComponent;
class UObject;
struct FFrame;

/** Settings to control the camera's lookat feature */
USTRUCT(BlueprintType)
struct FCameraLookatTrackingSettings
{
	GENERATED_USTRUCT_BODY()

	FCameraLookatTrackingSettings()
		: bEnableLookAtTracking(false)
		, bDrawDebugLookAtTrackingPosition(false)
		, LookAtTrackingInterpSpeed(0.f)
		, LastLookatTrackingRotation(FRotator::ZeroRotator)
		, LastLookatTrackingRotationFrame(-1)
		, RelativeOffset(FVector::ZeroVector)
		, bAllowRoll(false)
	{
	}

	/** True to enable lookat tracking, false otherwise. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "LookAt")
	uint8 bEnableLookAtTracking : 1;

	/** True to draw a debug representation of the lookat location */
	UPROPERTY(Transient, EditAnywhere, BlueprintReadWrite, Category = "LookAt")
	uint8 bDrawDebugLookAtTrackingPosition : 1;

	/** Controls degree of smoothing. 0.f for no smoothing, higher numbers for faster/tighter tracking. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "LookAt")
	float LookAtTrackingInterpSpeed;

	/** Last known lookat tracking rotation (used during interpolation) */
	FRotator LastLookatTrackingRotation;
	
	/** The frame we set the above on, so the camera component can restore it */
	uint32 LastLookatTrackingRotationFrame;

	/** If set, camera will track this actor's location */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "LookAt")
	TSoftObjectPtr<AActor> ActorToTrack;

	/** Offset from actor position to look at. Relative to actor if tracking an actor, relative to world otherwise. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "LookAt")
	FVector RelativeOffset;

	/** True to allow user-defined roll, false otherwise. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "LookAt")
	uint8 bAllowRoll : 1;
};

/** 
 * A CineCameraActor is a CameraActor specialized to work like a cinematic camera.
 */
UCLASS(ClassGroup = Common, hideCategories = (Input, Rendering, AutoPlayerActivation), showcategories = ("Input|MouseInput", "Input|TouchInput"), Blueprintable, MinimalAPI)
class ACineCameraActor : public ACameraActor
{
	GENERATED_BODY()

public:
	// Ctor
	CINEMATICCAMERA_API ACineCameraActor(const FObjectInitializer& ObjectInitializer);

	CINEMATICCAMERA_API virtual void Tick(float DeltaTime) override;
	CINEMATICCAMERA_API virtual bool ShouldTickIfViewportsOnly() const override;
	CINEMATICCAMERA_API virtual void PostInitializeComponents() override;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Current Camera Settings")
	FCameraLookatTrackingSettings LookatTrackingSettings;

	/** Returns the CineCameraComponent of this CineCamera */
	UFUNCTION(BlueprintCallable, Category="Camera")
	UCineCameraComponent* GetCineCameraComponent() const { return CineCameraComponent; }

protected:
	/** Set to true to skip any interpolations on the next update. Resets to false automatically. */
	uint8 bResetInterplation : 1;

	CINEMATICCAMERA_API FVector GetLookatLocation() const;

	CINEMATICCAMERA_API virtual void NotifyCameraCut() override;

	CINEMATICCAMERA_API bool ShouldTickForTracking() const;

private:
	/** Returns CineCameraComponent subobject **/
	class UCineCameraComponent* CineCameraComponent;
};
