// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Camera/CameraComponent.h"
#include "CineCameraSettings.h"
#include "CineCameraComponent.generated.h"

class AActor;
class UMaterial;
class UMaterialInstanceDynamic;
class UStaticMesh;
class UStaticMeshComponent;

/**
 * A specialized version of a camera component, geared toward cinematic usage.
 */
UCLASS(HideCategories = (CameraSettings), HideFunctions = (SetFieldOfView, SetAspectRatio), Blueprintable, ClassGroup = Camera, meta = (BlueprintSpawnableComponent), Config = Engine, MinimalAPI)
class UCineCameraComponent : public UCameraComponent
{
	GENERATED_BODY()

public:
	/** Default constuctor. */
	CINEMATICCAMERA_API UCineCameraComponent();

	CINEMATICCAMERA_API virtual void GetCameraView(float DeltaTime, FMinimalViewInfo& DesiredView) override;
#if WITH_EDITOR
	CINEMATICCAMERA_API virtual FText GetFilmbackText() const override;
#endif
	UPROPERTY()
	FCameraFilmbackSettings FilmbackSettings_DEPRECATED;

	/** Controls the filmback of the camera. */
	UPROPERTY(Interp, BlueprintSetter = SetFilmback, EditAnywhere, BlueprintReadWrite, Category = "Current Camera Settings")
	FCameraFilmbackSettings Filmback;

	UFUNCTION(BlueprintSetter)
	CINEMATICCAMERA_API void SetFilmback(const FCameraFilmbackSettings& NewFilmback);

	/** Controls the camera's lens. */
	UPROPERTY(EditAnywhere, BlueprintSetter = SetLensSettings, BlueprintReadWrite, Category = "Current Camera Settings")
	FCameraLensSettings LensSettings;

	UFUNCTION(BlueprintSetter)
	CINEMATICCAMERA_API void SetLensSettings(const FCameraLensSettings& NewLensSettings);

	/** Controls the camera's focus. */
	UPROPERTY(EditAnywhere, BlueprintSetter = SetFocusSettings, BlueprintReadWrite, Category = "Current Camera Settings")
	FCameraFocusSettings FocusSettings;

	UFUNCTION(BlueprintSetter)
	CINEMATICCAMERA_API void SetFocusSettings(const FCameraFocusSettings& NewFocusSettings);
	
	/** Controls the crop settings. */
	UPROPERTY(EditAnywhere, BlueprintSetter = SetCropSettings, BlueprintReadWrite, Category = "Current Camera Settings")
	FPlateCropSettings CropSettings;

	UFUNCTION(BlueprintSetter)
	CINEMATICCAMERA_API void SetCropSettings(const FPlateCropSettings& NewCropSettings);

	/** Current focal length of the camera (i.e. controls FoV, zoom) */
	UPROPERTY(Interp, BlueprintSetter = SetCurrentFocalLength, EditAnywhere, BlueprintReadWrite, Category = "Current Camera Settings")
	float CurrentFocalLength;

	/** Current aperture, in terms of f-stop (e.g. 2.8 for f/2.8) */
	UPROPERTY(Interp, BlueprintSetter = SetCurrentAperture, EditAnywhere, BlueprintReadWrite, Category = "Current Camera Settings")
	float CurrentAperture;

	UFUNCTION(BlueprintSetter)
	CINEMATICCAMERA_API void SetCurrentAperture(const float NewCurrentAperture);
	
	/** Read-only. Control this value via FocusSettings. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Current Camera Settings")
	float CurrentFocusDistance;

	/** Exposure method, for determining whether exposure-related post-process settings should be automatically set from camera parameters */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Current Camera Settings")
	ECameraExposureMethod ExposureMethod;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Current Camera Settings", meta = (InlineEditConditionToggle))
	uint32 bOverride_CustomNearClippingPlane:1;

	/** Set bOverride_CustomNearClippingPlane to true if you want to use a custom clipping plane instead of GNearClippingPlane. */
	UPROPERTY(Interp, BlueprintSetter = SetCustomNearClippingPlane, EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Current Camera Settings", meta = (UIMin = "0.00001", ClampMin = "0.00001", editcondition = "bOverride_CustomNearClippingPlane", Units = cm))
	float CustomNearClippingPlane;
	
#if WITH_EDITORONLY_DATA
	/** Read-only. Control this value with CurrentFocalLength (and filmback settings). */
	UPROPERTY(VisibleAnywhere, Category = "Current Camera Settings")
	float CurrentHorizontalFOV;
#endif

	/** Override setting FOV to manipulate Focal Length. */
	CINEMATICCAMERA_API virtual void SetFieldOfView(float InFieldOfView) override;
	
	UFUNCTION(BlueprintCallable, BlueprintSetter, Category = "Cine Camera")
	CINEMATICCAMERA_API void SetCurrentFocalLength(float InFocalLength);

	/** Sets near clipping plane of the cine camera. */
	UFUNCTION(BlueprintCallable, BlueprintSetter, Category = "Cine Camera")
	CINEMATICCAMERA_API void SetCustomNearClippingPlane(const float NewCustomNearClippingPlane);

	/** Returns the horizonal FOV of the camera with current settings. */
	UFUNCTION(BlueprintCallable, Category = "Cine Camera")
	CINEMATICCAMERA_API float GetHorizontalFieldOfView() const;
	
	/** Returns the vertical FOV of the camera with current settings. */
	UFUNCTION(BlueprintCallable, Category = "Cine Camera")
	CINEMATICCAMERA_API float GetVerticalFieldOfView() const;

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Cine Camera")
	CINEMATICCAMERA_API float GetHorizontalProjectionOffset() const;

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Cine Camera")
	CINEMATICCAMERA_API float GetVerticalProjectionOffset() const;
	
	/** Returns the filmback name of the camera with the current settings. */
	UFUNCTION(BlueprintCallable, Category = "Cine Camera")
	CINEMATICCAMERA_API FString GetFilmbackPresetName() const;

	/** Returns the name of the default filmback preset. */
	UE_DEPRECATED(5.1, "This function has been deprecated, please access the presets via the CineCameraSettings object.")
	UFUNCTION(BlueprintPure, Category = "Cine Camera")
	CINEMATICCAMERA_API FString GetDefaultFilmbackPresetName() const;

	/** Set the current preset settings by preset name. */
	UFUNCTION(BlueprintCallable, Category = "Cine Camera")
	CINEMATICCAMERA_API void SetFilmbackPresetByName(const FString& InPresetName);

	/** Returns the lens name of the camera with the current settings. */
	UFUNCTION(BlueprintCallable, Category = "Cine Camera")
	CINEMATICCAMERA_API FString GetLensPresetName() const;

	/** Set the current lens settings by preset name. */
	UFUNCTION(BlueprintCallable, Category = "Cine Camera")
	CINEMATICCAMERA_API void SetLensPresetByName(const FString& InPresetName);
	
	/** Returns the lens name of the camera with the current settings. */
	UFUNCTION(BlueprintCallable, Category = "Cine Camera")
	CINEMATICCAMERA_API FString GetCropPresetName() const;

	/** Set the current lens settings by preset name. */
	UFUNCTION(BlueprintCallable, Category = "Cine Camera")
	CINEMATICCAMERA_API void SetCropPresetByName(const FString& InPresetName);

	/** Returns a copy of the list of available filmback presets. */
	UE_DEPRECATED(5.1, "This function has been deprecated, please access the presets via the CineCameraSettings object.")
	UFUNCTION(BlueprintCallable, Category = "Cine Camera")
	static CINEMATICCAMERA_API TArray<FNamedFilmbackPreset> GetFilmbackPresetsCopy();

	/** Returns a copy of the list of available lens presets. */
	UE_DEPRECATED(5.1, "This function has been deprecated, please access the presets via the CineCameraSettings object.")
	UFUNCTION(BlueprintCallable, Category = "Cine Camera")
	static CINEMATICCAMERA_API TArray<FNamedLensPreset> GetLensPresetsCopy();

	/** Returns a list of available filmback presets. */
	UE_DEPRECATED(5.1, "This function has been deprecated, please access the presets via the CineCameraSettings object.")
	static CINEMATICCAMERA_API TArray<FNamedFilmbackPreset> const& GetFilmbackPresets();
	
	/** Returns a list of available lens presets. */
	UE_DEPRECATED(5.1, "This function has been deprecated, please access the presets via the CineCameraSettings object.")
	static CINEMATICCAMERA_API TArray<FNamedLensPreset> const& GetLensPresets();

#if WITH_EDITOR
	/** Update the debug focus plane position and orientation. */
	CINEMATICCAMERA_API void UpdateDebugFocusPlane();
#endif

	/** Returns the world to meters scale for the current UWorld */
	CINEMATICCAMERA_API float GetWorldToMetersScale() const;

protected:

	/** Most recent calculated focus distance. Used for interpolation. */
	float LastFocusDistance;

	/** Set to true to skip any interpolations on the next update. Resets to false automatically. */
	uint8 bResetInterpolation : 1;

	/// @cond DOXYGEN_WARNINGS
	
	CINEMATICCAMERA_API virtual void PostLoad() override;
	
	/// @endcond
	
	CINEMATICCAMERA_API virtual void Serialize(FArchive& Ar) override;
	CINEMATICCAMERA_API virtual void PostInitProperties() override;
	CINEMATICCAMERA_API virtual void OnRegister() override;
	CINEMATICCAMERA_API virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;

#if WITH_EDITOR
	CINEMATICCAMERA_API virtual void OnComponentDestroyed(bool bDestroyingHierarchy) override;
#endif

#if WITH_EDITORONLY_DATA
	/** Mesh used for debug focus plane visualization */
	UPROPERTY(transient)
	TObjectPtr<UStaticMesh> FocusPlaneVisualizationMesh;

	/** Material used for debug focus plane visualization */
	UPROPERTY(transient)
	TObjectPtr<UMaterial> FocusPlaneVisualizationMaterial;

	/** Component for the debug focus plane visualization */
	UPROPERTY(transient)
	TObjectPtr<UStaticMeshComponent> DebugFocusPlaneComponent;

	/** Dynamic material instance for the debug focus plane visualization */
	UPROPERTY(transient)
	TObjectPtr<UMaterialInstanceDynamic> DebugFocusPlaneMID;

	CINEMATICCAMERA_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	CINEMATICCAMERA_API virtual void ResetProxyMeshTransform() override;
#endif

	/** List of available filmback presets */
	UE_DEPRECATED(5.1, "This property is now located on the UCineCameraSettings object")
	UPROPERTY(config)
	TArray<FNamedFilmbackPreset> FilmbackPresets;

	/** List of available lens presets */
	UE_DEPRECATED(5.1, "This property is now located on the UCineCameraSettings object")
	UPROPERTY(config)
	TArray<FNamedLensPreset> LensPresets;

	/** Deprecated. See DefaultFilmbackPreset */
	UE_DEPRECATED(5.1, "This property has been removed and fully replaced by DefaultFilmbackPreset on the UCineCameraSettings object")
	UPROPERTY(config)
	FString DefaultFilmbackPresetName_DEPRECATED;

	/** Name of the default filmback preset */
	UE_DEPRECATED(5.1, "This property is now located on the UCineCameraSettings object")
	UPROPERTY(config)
	FString DefaultFilmbackPreset;

	/** Name of the default lens preset */
	UE_DEPRECATED(5.1, "This property is now located on the UCineCameraSettings object")
	UPROPERTY(config)
	FString DefaultLensPresetName;

	/** Default focal length (will be constrained by default lens) */
	UE_DEPRECATED(5.1, "This property is now located on the UCineCameraSettings object")
	UPROPERTY(config)
	float DefaultLensFocalLength;
	
	/** Default aperture (will be constrained by default lens) */
	UE_DEPRECATED(5.1, "This property is now located on the UCineCameraSettings object")
	UPROPERTY(config)
	float DefaultLensFStop;

	CINEMATICCAMERA_API virtual void UpdateCameraLens(float DeltaTime, FMinimalViewInfo& DesiredView);

	CINEMATICCAMERA_API virtual void NotifyCameraCut() override;
	
	CINEMATICCAMERA_API void RecalcDerivedData();

private:
	float GetDesiredFocusDistance(const FVector& InLocation) const;
	void SetLensPresetByNameInternal(const FString& InPresetName);
	void SetFilmbackPresetByNameInternal(const FString& InPresetName, FCameraFilmbackSettings& InOutFilmbackSettings);
	void SetCropPresetByNameInternal(const FString& InPresetName);
	float GetHorizontalFieldOfViewInternal(bool bIncludeOverscan = true) const;
	float GetVerticalFieldOfViewInternal(bool bIncludeOverscan) const;
	
#if WITH_EDITORONLY_DATA
	void CreateDebugFocusPlane();
	void DestroyDebugFocusPlane();
#endif
};
