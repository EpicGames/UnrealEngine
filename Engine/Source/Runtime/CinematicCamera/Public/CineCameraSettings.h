// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "Engine/Scene.h"
#include "Templates/SharedPointer.h"
#include "CineCameraSettings.generated.h"

class SNotificationItem;

/** #note, this struct has a details customization in CameraFilmbackSettingsCustomization.cpp/h */
USTRUCT(BlueprintType)
struct FCameraFilmbackSettings
{
	GENERATED_USTRUCT_BODY()

	/** Horizontal size of filmback or digital sensor, in mm. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Filmback", meta = (ClampMin = "0.001", ForceUnits = mm))
	float SensorWidth;

	/** Vertical size of filmback or digital sensor, in mm. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Filmback", meta = (ClampMin = "0.001", ForceUnits = mm))
	float SensorHeight;

	/** Horizontal offset of the filmback, in mm. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Filmback", meta = (ForceUnits = mm))
	float SensorHorizontalOffset;

	/** Vertical offset of the filmback, in mm. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Filmback", meta = (ForceUnits = mm))
	float SensorVerticalOffset;
	
	/** Read-only. Computed from Sensor dimensions. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Filmback")
	float SensorAspectRatio;

	bool operator==(const FCameraFilmbackSettings& Other) const
	{
		return (SensorWidth == Other.SensorWidth)
			&& (SensorHeight == Other.SensorHeight)
			&& (SensorHorizontalOffset == Other.SensorHorizontalOffset)
			&& (SensorVerticalOffset == Other.SensorVerticalOffset);
	}

	bool operator!=(const FCameraFilmbackSettings& Other) const
	{
		return !operator==(Other);
	}

	void RecalcSensorAspectRatio()
	{
		SensorAspectRatio = (SensorHeight > 0.f) ? (SensorWidth / SensorHeight) : 0.f;
	}

	FCameraFilmbackSettings()
		: SensorWidth(24.89f)
		, SensorHeight(18.67f)
		, SensorHorizontalOffset(0.0f)
		, SensorVerticalOffset(0.0f)
		, SensorAspectRatio(1.33f)
	{
	}
};

/** A named bundle of filmback settings used to implement filmback presets */
USTRUCT(BlueprintType)
struct FNamedFilmbackPreset
{
	GENERATED_USTRUCT_BODY()

	/** Name for the preset. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Filmback")
	FString Name;

	/** Display name for the preset. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Filmback")
	FText DisplayName;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Filmback")
	FCameraFilmbackSettings FilmbackSettings;
};

/** 
 * #note, this struct has a details customization in CameraLensSettingsCustomization.cpp/h
 */
USTRUCT(BlueprintType)
struct FCameraLensSettings
{
	GENERATED_USTRUCT_BODY()

	/** Default constructor, initializing with default values */
	FCameraLensSettings()
		: MinFocalLength(50.f)
		, MaxFocalLength(50.f)
		, MinFStop(2.f)
		, MaxFStop(2.f)
		, MinimumFocusDistance(15.f)
		, DiaphragmBladeCount(FPostProcessSettings::kDefaultDepthOfFieldBladeCount)
	{
	}

	/** Minimum focal length for this lens */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lens", meta = (ForceUnits = mm, ClampMin = "0.001"))
	float MinFocalLength = 0.f;

	/** Maximum focal length for this lens */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lens", meta = (ForceUnits = mm, ClampMin = "0.001"))
	float MaxFocalLength = 0.f;

	/** Minimum aperture for this lens (e.g. 2.8 for an f/2.8 lens) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lens")
	float MinFStop = 0.f;

	/** Maximum aperture for this lens (e.g. 2.8 for an f/2.8 lens) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lens")
	float MaxFStop = 0.f;

	/** Shortest distance this lens can focus on. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lens", meta = (ForceUnits = mm))
	float MinimumFocusDistance = 0.f;

	/** Squeeze factor for anamorphic lenses. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lens", meta = (ClampMin = "1", ClampMax = "2"))
	float SqueezeFactor = 1.f;

	/** Number of blades of diaphragm. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lens", meta = (ClampMin = "4", ClampMax = "16"))
	int32 DiaphragmBladeCount = 0;

	bool operator==(const FCameraLensSettings& Other) const
	{
		return (MinFocalLength == Other.MinFocalLength)
			&& (MaxFocalLength == Other.MaxFocalLength)
			&& (MinFStop == Other.MinFStop)
			&& (MaxFStop == Other.MaxFStop)
			&& (MinimumFocusDistance == Other.MinimumFocusDistance)
			&& (SqueezeFactor == Other.SqueezeFactor)
			&& (DiaphragmBladeCount == Other.DiaphragmBladeCount);
	}
};

/** A named bundle of lens settings used to implement lens presets. */
USTRUCT(BlueprintType)
struct FNamedLensPreset
{
	GENERATED_USTRUCT_BODY()

	/** Name for the preset. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Lens")
	FString Name;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Lens")
	FCameraLensSettings LensSettings;
};

/** 
 * #note, this struct has a details customization in CameraCropSettingsCustomization.cpp/h
 */
USTRUCT(BlueprintType)
struct FPlateCropSettings
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Crop", meta=(DisplayName = "Cropped Aspect Ratio"))
	float AspectRatio = 0.f;

	bool operator==(const FPlateCropSettings& Other) const
	{
		return (AspectRatio == Other.AspectRatio);
	}
};

/** A named bundle of crop settings used to implement crop presets. */
USTRUCT(BlueprintType)
struct FNamedPlateCropPreset
{
	GENERATED_USTRUCT_BODY()

	/** Name for the preset. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Crop")
	FString Name;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Crop")
	FPlateCropSettings CropSettings;
};

/** Supported methods for focusing the camera. */
UENUM()
enum class ECameraFocusMethod : uint8
{
	/** Don't override, ie. allow post process volume settings to persist. */
	DoNotOverride,

	/** Allows for specifying or animating exact focus distances. */
	Manual,

	/** Locks focus to specific object. */
	Tracking,

	/** Disable depth of field entirely. */
	Disable,

	MAX UMETA(Hidden)
};

/** Settings to control tracking-focus mode. */
USTRUCT(BlueprintType)
struct FCameraTrackingFocusSettings
{
	GENERATED_USTRUCT_BODY()

	/** Focus distance will be tied to this actor's location. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Tracking Focus")
	TSoftObjectPtr<AActor> ActorToTrack;

	/** Offset from actor position to track. Relative to actor if tracking an actor, relative to world otherwise. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Tracking Focus")
	FVector RelativeOffset;

	/** True to draw a debug representation of the tracked position. */
	UPROPERTY(Transient, EditAnywhere, BlueprintReadWrite, Category = "Tracking Focus")
	uint8 bDrawDebugTrackingFocusPoint : 1;

	FCameraTrackingFocusSettings()
		: RelativeOffset(ForceInitToZero),
		bDrawDebugTrackingFocusPoint(false)
	{}
};

/** Settings to control camera focus */
USTRUCT(BlueprintType)
struct FCameraFocusSettings
{
	GENERATED_USTRUCT_BODY()

	/** Which method to use to handle camera focus */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Focus Method")
	ECameraFocusMethod FocusMethod;
	
	/** Manually-controlled focus distance (manual focus mode only) */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Manual Focus Settings", meta=(Units=cm))
	float ManualFocusDistance;

	/** Settings to control tracking focus (tracking focus mode only) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tracking Focus Settings")
	FCameraTrackingFocusSettings TrackingFocusSettings;

#if WITH_EDITORONLY_DATA
	/** True to draw a translucent plane at the current focus depth, for easy tweaking. */
	UPROPERTY(Transient, EditAnywhere, Category = "Focus Settings")
	uint8 bDrawDebugFocusPlane : 1;

	/** For customizing the focus plane color, in case the default doesn't show up well in your scene. */
	UPROPERTY(EditAnywhere, Category = "Focus Settings", meta = (EditCondition = "bDrawDebugFocusPlane"))
	FColor DebugFocusPlaneColor;
#endif 

	/** True to use interpolation to smooth out changes in focus distance, false for focus distance changes to be instantaneous. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Focus Settings")
	uint8 bSmoothFocusChanges : 1;
	
	/** Controls interpolation speed when smoothing focus distance changes. Ignored if bSmoothFocusChanges is false. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Focus Settings")
	float FocusSmoothingInterpSpeed;

	/** Additional focus depth offset, used for manually tweaking if your chosen focus method needs adjustment */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Focus Settings")
	float FocusOffset;

	FCameraFocusSettings() : 
		FocusMethod(ECameraFocusMethod::Manual),
		ManualFocusDistance(100000.f),
		TrackingFocusSettings(),
#if WITH_EDITORONLY_DATA
		bDrawDebugFocusPlane(false),
		DebugFocusPlaneColor(102, 26, 204, 153),		// purple
#endif
		bSmoothFocusChanges(false),
		FocusSmoothingInterpSpeed(8.f),
		FocusOffset(0.f)
	{}
};

/** Supported methods for exposure handling. */
UENUM()
enum class ECameraExposureMethod : uint8
{
	/** Don't override, ie. allow post process volume settings to persist. */
	DoNotOverride,

	/** Set exposure settings based on the current lens and filmback settings. */
	Enabled,

	MAX UMETA(Hidden)
};

UCLASS(config=Engine, defaultconfig, meta=(DisplayName="Cinematic Camera"), MinimalAPI)
class UCineCameraSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	CINEMATICCAMERA_API virtual void PostInitProperties() override;

#if WITH_EDITOR
	CINEMATICCAMERA_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	/** Name of the default lens preset */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, BlueprintSetter=SetDefaultLensPresetName, Category=Lens, meta=(GetOptions=GetLensPresetNames))
	FString DefaultLensPresetName;
	
	/** Default focal length (will be constrained by default lens) */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, BlueprintSetter=SetDefaultLensFocalLength, Category=Lens)
	float DefaultLensFocalLength;
	
	/** Default aperture (will be constrained by default lens) */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, BlueprintSetter=SetDefaultLensFStop, Category=Lens)
	float DefaultLensFStop;
	
	/** List of available lens presets */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, BlueprintSetter=SetLensPresets, Category=Lens, meta=(TitleProperty=Name))
	TArray<FNamedLensPreset> LensPresets;

	/** Name of the default filmback preset */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, BlueprintSetter=SetDefaultfilmbackPreset, Category=Filmback, meta=(GetOptions=GetFilmbackPresetNames))
	FString DefaultFilmbackPreset;

	/** List of available filmback presets */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, BlueprintSetter=SetFilmbackPresets, Category=Filmback, meta=(TitleProperty=Name))
	TArray<FNamedFilmbackPreset> FilmbackPresets;

	/** Name of the default crop preset */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, BlueprintSetter=SetDefaultCropPresetName, Category=Crop, meta=(GetOptions=GetCropPresetNames))
	FString DefaultCropPresetName;
	
	/** List of available crop presets */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, BlueprintSetter=SetCropPresets, Category=Crop, meta=(TitleProperty=Name))
	TArray<FNamedPlateCropPreset> CropPresets;

	static CINEMATICCAMERA_API TArray<FNamedLensPreset> const& GetLensPresets();
	static CINEMATICCAMERA_API TArray<FNamedFilmbackPreset> const& GetFilmbackPresets();
	static CINEMATICCAMERA_API TArray<FNamedPlateCropPreset> const& GetCropPresets();

	// Gets the Lens settings associated with a given preset name
	// Returns true if a preset with the given name was found
	UFUNCTION(BlueprintCallable, Category=Cinematics, meta=(ReturnDisplayName = "Success"))
	CINEMATICCAMERA_API bool GetLensPresetByName(const FString PresetName, FCameraLensSettings& LensSettings);

	// Gets the Filmback settings associated with a given preset name
	// Returns true if a preset with the given name was found
	UFUNCTION(BlueprintCallable, Category=Cinematics, meta=(ReturnDisplayName = "Success"))
	CINEMATICCAMERA_API bool GetFilmbackPresetByName(const FString PresetName, FCameraFilmbackSettings& FilmbackSettings);

	// Gets the Crop settings associated with a given preset name
	// Returns true if a preset with the given name was found
	UFUNCTION(BlueprintCallable, Category=Cinematics, meta=(ReturnDisplayName = "Success"))
	CINEMATICCAMERA_API bool GetCropPresetByName(const FString PresetName, FPlateCropSettings& CropSettings);

private:
	UFUNCTION(BlueprintCallable, Category=Cinematics)
	static UCineCameraSettings* GetCineCameraSettings();

	/* Internal Blueprint Setter functions that call SaveConfig after setting the variable to ensure settings persist */
	
	UFUNCTION(BlueprintSetter)
	void SetDefaultLensPresetName(const FString InDefaultLensPresetName);

	UFUNCTION(BlueprintSetter)
	void SetDefaultLensFocalLength(const float InDefaultLensFocalLength);

	UFUNCTION(BlueprintSetter)
	void SetDefaultLensFStop(const float InDefaultLensFStop);

	UFUNCTION(BlueprintSetter)
	void SetLensPresets(const TArray<FNamedLensPreset>& InLensPresets);

	UFUNCTION(BlueprintSetter)
	void SetDefaultFilmbackPreset(const FString InDefaultFilmbackPreset);

	UFUNCTION(BlueprintSetter)
	void SetFilmbackPresets(const TArray<FNamedFilmbackPreset>& InFilmbackPresets);

	UFUNCTION(BlueprintSetter)
	void SetDefaultCropPresetName(const FString InDefaultCropPresetName);

	UFUNCTION(BlueprintSetter)
	void SetCropPresets(const TArray<FNamedPlateCropPreset>& InCropPresets);

	/* Functions used for the GetOptions metadata */
	
	UFUNCTION()
	TArray<FString> GetLensPresetNames() const;

	UFUNCTION()
	TArray<FString> GetFilmbackPresetNames() const;

	UFUNCTION()
	TArray<FString> GetCropPresetNames() const;

	static const FString CineCameraConfigSection;
	TSharedPtr<SNotificationItem> Notification;
	void CopyOldConfigSettings();
	void CloseNotification();

	// Sensor aspect ratio is derived from Sensor Height and Width so needs to be updated when either property is updated
	void RecalcSensorAspectRatios();
};
